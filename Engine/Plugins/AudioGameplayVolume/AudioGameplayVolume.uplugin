{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "AudioGameplayVolume",
	"Description": "Audio Gameplay Volume Plugin",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AudioGameplayVolume",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AudioGameplayVolumeEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins" : [
		{
			"Name": "AudioGameplay",
			"Enabled": true
		}
	]
}