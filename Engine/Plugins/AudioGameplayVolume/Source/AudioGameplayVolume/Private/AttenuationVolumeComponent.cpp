// Copyright Epic Games, Inc. All Rights Reserved.

#include "AttenuationVolumeComponent.h"
#include "Sound/AudioVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AttenuationVolumeComponent)

constexpr TCHAR FProxyMutator_Attenuation::MutatorAttenuationName[];

FProxyMutator_Attenuation::FProxyMutator_Attenuation()
{
	MutatorName = MutatorAttenuationName;
}

void FProxyMutator_Attenuation::Apply(FInteriorSettings& InteriorSettings) const
{
	FProxyVolumeMutator::Apply(InteriorSettings);

	InteriorSettings.ExteriorVolume = ExteriorVolume;
	InteriorSettings.ExteriorTime = ExteriorTime;
	InteriorSettings.InteriorVolume = InteriorVolume;
	InteriorSettings.InteriorTime = InteriorTime;
}

void FProxyMutator_Attenuation::Apply(FAudioProxyActiveSoundParams& Params) const
{
	Params.bAffectedByAttenuation |= Params.bListenerInVolume;
}

UAttenuationVolumeComponent::UAttenuationVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PayloadType = AudioGameplay::EComponentPayload::AGCP_ActiveSound | AudioGameplay::EComponentPayload::AGCP_Listener;
	bAutoActivate = true;
}

void UAttenuationVolumeComponent::SetExteriorVolume(float Volume, float InterpolateTime)
{
	ExteriorVolume = Volume;
	ExteriorTime = InterpolateTime;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

void UAttenuationVolumeComponent::SetInteriorVolume(float Volume, float InterpolateTime)
{
	InteriorVolume = Volume;
	InteriorTime = InterpolateTime;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> UAttenuationVolumeComponent::FactoryMutator() const
{
	return MakeShared<FProxyMutator_Attenuation>();
}

void UAttenuationVolumeComponent::CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	TSharedPtr<FProxyMutator_Attenuation> AttenuationMutator = StaticCastSharedPtr<FProxyMutator_Attenuation>(Mutator);
	AttenuationMutator->ExteriorVolume = ExteriorVolume;
	AttenuationMutator->ExteriorTime = ExteriorTime;
	AttenuationMutator->InteriorVolume = InteriorVolume;
	AttenuationMutator->InteriorTime = InteriorTime;
}

