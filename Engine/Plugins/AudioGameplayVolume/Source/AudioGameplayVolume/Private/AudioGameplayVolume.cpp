// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolume.h"
#include "AudioGameplayVolumeProxy.h"
#include "AudioGameplayVolumeComponent.h"
#include "Components/BrushComponent.h"
#include "Engine/CollisionProfile.h"
#include "Net/UnrealNetwork.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayVolume)

AAudioGameplayVolume::AAudioGameplayVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	if (UBrushComponent* BrushComp = GetBrushComponent())
	{
		BrushComp->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		BrushComp->SetGenerateOverlapEvents(false);
		BrushComp->bAlwaysCreatePhysicsState = true;
	}

	AGVComponent = CreateDefaultSubobject<UAudioGameplayVolumeComponent>(TEXT("AGVComponent"));

#if WITH_EDITOR
	bColored = true;
	BrushColor = FColor(255, 255, 0, 255);
#endif // WITH_EDITOR
}

void AAudioGameplayVolume::SetEnabled(bool bEnable)
{
	if (bEnable != bEnabled)
	{
		bEnabled = bEnable;
		if (CanSupportProxy())
		{
			AddProxy();
		}
		else
		{
			RemoveProxy();
		}
	}
}

void AAudioGameplayVolume::OnListenerEnter_Implementation()
{
	OnListenerEnterEvent.Broadcast();
}

void AAudioGameplayVolume::OnListenerExit_Implementation()
{
	OnListenerExitEvent.Broadcast();
}

#if WITH_EDITOR
void AAudioGameplayVolume::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AAudioGameplayVolume, bEnabled))
	{
		if (CanSupportProxy())
		{
			AddProxy();
		}
		else
		{
			RemoveProxy();
		}
	}
}
#endif // WITH_EDITOR

void AAudioGameplayVolume::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	DOREPLIFETIME(AAudioGameplayVolume, bEnabled);
}

void AAudioGameplayVolume::PostInitializeComponents()
{
	Super::PostInitializeComponents();

	if (CanSupportProxy())
	{
		AddProxy();
	}
}

void AAudioGameplayVolume::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (USceneComponent* SC = GetRootComponent())
	{
		SC->TransformUpdated.AddUObject(this, &AAudioGameplayVolume::TransformUpdated);
	}

	if (AGVComponent)
	{
		UAGVPrimitiveComponentProxy* PrimitiveComponentProxy = Cast<UAGVPrimitiveComponentProxy>(AGVComponent->GetProxy());
		if (!PrimitiveComponentProxy)
		{
			AGVComponent->SetProxy(NewObject<UAGVPrimitiveComponentProxy>(AGVComponent));
		}

		AGVComponent->bAutoActivate = false;
	}
}

void AAudioGameplayVolume::PostUnregisterAllComponents()
{
	RemoveProxy();

	// Component can be nulled due to GC at this point
	if (USceneComponent* SC = GetRootComponent())
	{
		SC->TransformUpdated.RemoveAll(this);
	}

	Super::PostUnregisterAllComponents();
}

void AAudioGameplayVolume::OnComponentDataChanged()
{
	if (CanSupportProxy())
	{
		UpdateProxy();
	}
}

bool AAudioGameplayVolume::CanSupportProxy() const
{
	if (!bEnabled || !AGVComponent || !AGVComponent->GetProxy())
	{
		return false;
	}

	return true;
}

void AAudioGameplayVolume::OnRep_bEnabled()
{
	if (CanSupportProxy())
	{
		AddProxy();
	}
	else
	{
		RemoveProxy();
	}
}

void AAudioGameplayVolume::TransformUpdated(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	UpdateProxy();
}

void AAudioGameplayVolume::AddProxy() const
{
	if (AGVComponent)
	{
		AGVComponent->OnProxyEnter.AddUniqueDynamic(const_cast<AAudioGameplayVolume*>(this), &AAudioGameplayVolume::OnListenerEnter);
		AGVComponent->OnProxyExit.AddUniqueDynamic(const_cast<AAudioGameplayVolume*>(this), &AAudioGameplayVolume::OnListenerExit);
		AGVComponent->Activate();
	}
}

void AAudioGameplayVolume::RemoveProxy() const
{
	if (AGVComponent)
	{
		AGVComponent->Deactivate();
		AGVComponent->OnProxyEnter.RemoveAll(this);
		AGVComponent->OnProxyExit.RemoveAll(this);
	}
}

void AAudioGameplayVolume::UpdateProxy() const
{
	if (AGVComponent)
	{
		AGVComponent->OnComponentDataChanged();
	}
}
