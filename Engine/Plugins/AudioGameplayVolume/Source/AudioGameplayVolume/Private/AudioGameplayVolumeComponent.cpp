// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeComponent.h"
#include "AudioGameplayVolumeProxy.h"
#include "AudioGameplayVolumeSubsystem.h"
#include "AudioDevice.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayVolumeComponent)

namespace AudioGameplayVolumeComponentVariables
{
	int32 bDuplicateProxy = 1;
	FAutoConsoleVariableRef CVarProxyDistanceCulling(
		TEXT("au.AudioGameplayVolumes.DuplicateProxy"),
		bDuplicateProxy,
		TEXT("Prevent cluster verification to fail by duplicating volume proxy if spawned actor references an existing volume proxy - 0: Disable, 1: Enable (default)"),
		ECVF_Default);
}

UAudioGameplayVolumeComponent::UAudioGameplayVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
}

void UAudioGameplayVolumeComponent::SetProxy(UAudioGameplayVolumeProxy* NewProxy)
{
	RemoveProxy();
	Proxy = NewProxy;

	if (IsActive())
	{
		AddProxy();
	}
}

void UAudioGameplayVolumeComponent::OnComponentDataChanged()
{
	if (IsActive())
	{
		UpdateProxy();
	}
}

void UAudioGameplayVolumeComponent::EnterProxy() const
{
	TInlineComponentArray<UActorComponent*> ActorComponents(GetOwner());
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		if (ActorComponent && ActorComponent->Implements<UAudioGameplayVolumeInteraction>())
		{
			IAudioGameplayVolumeInteraction::Execute_OnListenerEnter(ActorComponent);
		}
	}

	OnProxyEnter.Broadcast();
}

void UAudioGameplayVolumeComponent::ExitProxy() const
{
	TInlineComponentArray<UActorComponent*> ActorComponents(GetOwner());
	for (UActorComponent* ActorComponent : ActorComponents)
	{
		if (ActorComponent && ActorComponent->Implements<UAudioGameplayVolumeInteraction>())
		{
			IAudioGameplayVolumeInteraction::Execute_OnListenerExit(ActorComponent);
		}
	}

	OnProxyExit.Broadcast();
}

#if WITH_EDITOR
void UAudioGameplayVolumeComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UAudioGameplayVolumeComponent, Proxy))
	{
		RemoveProxy();

		if (IsActive())
		{
			AddProxy();
		}
	}
}
#endif // WITH_EDITOR

void UAudioGameplayVolumeComponent::OnRegister()
{
	Super::OnRegister();

	if (AudioGameplayVolumeComponentVariables::bDuplicateProxy)
	{
		if (Proxy != nullptr && Proxy->IsTemplate())
		{
			Proxy = DuplicateObject<UAudioGameplayVolumeProxy>(Proxy, this);
		}
	}
}

void UAudioGameplayVolumeComponent::OnUnregister()
{
	Super::OnUnregister();
	RemoveProxy();
}

void UAudioGameplayVolumeComponent::Enable()
{
	if (Proxy != nullptr)
	{
		Super::Enable();
		AddProxy();
	}
}

void UAudioGameplayVolumeComponent::Disable()
{
	RemoveProxy();
	Super::Disable();
}

void UAudioGameplayVolumeComponent::AddProxy() const
{
	if (UAudioGameplayVolumeSubsystem* VolumeSubsystem = GetSubsystem())
	{
		VolumeSubsystem->AddVolumeComponent(this);
	}
}

void UAudioGameplayVolumeComponent::RemoveProxy() const
{
	if (UAudioGameplayVolumeSubsystem* VolumeSubsystem = GetSubsystem())
	{
		VolumeSubsystem->RemoveVolumeComponent(this);
	}
}

void UAudioGameplayVolumeComponent::UpdateProxy() const
{
	if (UAudioGameplayVolumeSubsystem* VolumeSubsystem = GetSubsystem())
	{
		VolumeSubsystem->UpdateVolumeComponent(this);
	}
}

UAudioGameplayVolumeSubsystem* UAudioGameplayVolumeComponent::GetSubsystem() const
{
	if (UWorld* World = GetWorld())
	{
		return FAudioDevice::GetSubsystem<UAudioGameplayVolumeSubsystem>(World->GetAudioDevice());
	}

	return nullptr;
}

UAudioGameplayVolumeComponentBase::UAudioGameplayVolumeComponentBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bAutoActivate = true;
}

