// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeListener.h"
#include "AudioGameplayVolumeMutator.h"
#include "AudioGameplayVolumeSubsystem.h"
#include "Misc/App.h"

namespace AudioGameplayVolumeListenerCVars
{
	int32 bFilterListenerMutatorsByPriority = 1;
	FAutoConsoleVariableRef CVarFilterListenerMutatorsByPriority(
		TEXT("au.AudioGameplayVolumes.FilterListenerMutatorsByPriority"),
		bFilterListenerMutatorsByPriority,
		TEXT("When non-zero, listener-applied mutators respect priority (only mutators with the highest priority level of each type are applied).\n")
		TEXT("When zero, uses legacy behavior where all mutators are applied regardless of priority.\n")
		TEXT("0: Disable (legacy), 1: Enable (default)"),
		ECVF_Default);
} // namespace AudioGameplayVolumeListenerCVars

void FInterpolatedInteriorSettings::Apply(const FInteriorSettings& NewSettings)
{
	if (InteriorSettings != NewSettings)
	{
		InteriorStartTime = FApp::GetCurrentTime();
		if (NewSettings.bIsWorldSettings)
		{
			// If we're going to default world settings, use our previous / current interpolation times.
			InteriorEndTime = InteriorStartTime + InteriorSettings.InteriorTime;
			ExteriorEndTime = InteriorStartTime + InteriorSettings.ExteriorTime;
			InteriorLPFEndTime = InteriorStartTime + InteriorSettings.InteriorLPFTime;
			ExteriorLPFEndTime = InteriorStartTime + InteriorSettings.ExteriorLPFTime;
		}
		else
		{
			InteriorEndTime = InteriorStartTime + NewSettings.InteriorTime;
			ExteriorEndTime = InteriorStartTime + NewSettings.ExteriorTime;
			InteriorLPFEndTime = InteriorStartTime + NewSettings.InteriorLPFTime;
			ExteriorLPFEndTime = InteriorStartTime + NewSettings.ExteriorLPFTime;
		}

		InteriorSettings = NewSettings;
	}
}

float FInterpolatedInteriorSettings::Interpolate(double CurrentTime, double EndTime) const
{
	if (CurrentTime < InteriorStartTime)
	{
		return 0.0f;
	}

	if (CurrentTime >= EndTime)
	{
		return 1.0f;
	}

	float InterpValue = (float)((CurrentTime - InteriorStartTime) / (EndTime - InteriorStartTime));
	return FMath::Clamp(InterpValue, 0.0f, 1.0f);
}

void FInterpolatedInteriorSettings::UpdateInteriorValues()
{
	// Store the interpolation value, not the actual value
	double CurrentTime = FApp::GetCurrentTime();
	InteriorVolumeInterp = Interpolate(CurrentTime, InteriorEndTime);
	ExteriorVolumeInterp = Interpolate(CurrentTime, ExteriorEndTime);
	InteriorLPFInterp = Interpolate(CurrentTime, InteriorLPFEndTime);
	ExteriorLPFInterp = Interpolate(CurrentTime, ExteriorLPFEndTime);
}

void FAudioGameplayVolumeListener::Update(const FAudioProxyMutatorSearchResult& Result, const FVector& InPosition, uint32 InDeviceId)
{
	check(IsInAudioThread());

	Position = InPosition;

	PreviousProxies = MoveTemp(CurrentProxies);
	CurrentProxies = Result.VolumeSet;

	TSet<uint32> EnteredProxies = CurrentProxies.Difference(PreviousProxies);
	TSet<uint32> ExitedProxies = PreviousProxies.Difference(CurrentProxies);

	// Remove the mutators we were previously affected by and add the new ones if:
	// We are a new listener to this audio device OR if we have entered or exited any proxies
	bool bMutatorsOutOfDate = (OwningDeviceId != InDeviceId) || EnteredProxies.Num() || ExitedProxies.Num();

	if (bMutatorsOutOfDate)
	{
		OwningDeviceId = InDeviceId;

		// Call remove on all the mutators we were previously affected by 
		for (const TSharedPtr<FProxyVolumeMutator>& Mutator : ActiveMutators)
		{
			Mutator->Remove(*this);
		}

		// Now cache and apply the currently matching mutators.
		// Mutators are sorted by type (name) and descending priority, so we can quickly apply the highest priority mutators.
		ActiveMutators = Result.MatchingMutators;
		if (AudioGameplayVolumeListenerCVars::bFilterListenerMutatorsByPriority)
		{
			FName MutatorGroup;
			int32 CurrentPriority = INDEX_NONE;
			for (const TSharedPtr<FProxyVolumeMutator>& Mutator : ActiveMutators)
			{
				// If we encounter a new name, it's a new type - with the highest priority for that group
				if (!Mutator->MutatorName.IsEqual(MutatorGroup))
				{
					MutatorGroup = Mutator->MutatorName;
					CurrentPriority = Mutator->Priority;
				}

				// Lower priority mutators are ignored, we need to advance to the next named group
				if (Mutator->Priority < CurrentPriority)
				{
					continue;
				}

				Mutator->Apply(*this);
			}
		}
		else
		{
			// Legacy behavior which will be removed -- apply all mutators in reverse priority order.
			for (const TSharedPtr<FProxyVolumeMutator>& Mutator : ActiveMutators)
			{
				Mutator->Apply(*this);
			}
		}

		// Reapply interior from mutators
		InteriorSettings.Apply(Result.InteriorSettings);
	}

	// Update interpolation
	InteriorSettings.UpdateInteriorValues();
}
