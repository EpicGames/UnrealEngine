// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeModule.h"
#include "AudioGameplayVolumeLogs.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "FAudioGameplayVolume"

DEFINE_LOG_CATEGORY(AudioGameplayVolumeLog);

void FAudioGameplayVolumeModule::StartupModule()
{
}

void FAudioGameplayVolumeModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FAudioGameplayVolumeModule, AudioGameplayVolume)
