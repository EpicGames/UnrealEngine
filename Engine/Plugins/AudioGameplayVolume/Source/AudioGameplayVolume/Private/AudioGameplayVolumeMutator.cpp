// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeMutator.h"
#include "AudioGameplayVolumeComponent.h"
#include "AudioGameplayVolumeSubsystem.h"
#include "Sound/SoundSubmixSend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayVolumeMutator)

FAudioProxyActiveSoundParams::FAudioProxyActiveSoundParams(const FAudioGameplayActiveSoundInfo& SoundInfo, const FAudioGameplayVolumeListener& InListener)
	: SourceInteriorVolume(SoundInfo.SourceInteriorVolume)
	, SourceInteriorLPF(SoundInfo.SourceInteriorLPF)
	, bUsingWorldSettings(SoundInfo.InteriorSettings.IsWorldSettings())
	, Listener(InListener)
	, Sound(SoundInfo)
{
}

void FAudioProxyActiveSoundParams::UpdateInteriorValues()
{
	const FInterpolatedInteriorSettings& ListenerInterior = Listener.GetInteriorSettings();

	// When a listener or a sound changes interior settings, we smoothly transition from the old settings
	// to the new settings.  We track both the listener and the sound's interpolation progress.
	// We use the interpolation progress from the object that has _most recently_ changed interior settings.
	// This allows moving sounds to smoothly interpolate across interior boundaries after the listener has completed 
	// transitioning to it's current interior settings (otherwise we'd abruptly slam the moving sound's interior settings to match the listener's).
	float ExtVolumeInterp = ListenerInterior.GetExteriorVolumeInterp();
	float IntVolumeInterp = ListenerInterior.GetInteriorVolumeInterp();
	float ExtLowPassInterp = ListenerInterior.GetExteriorLPFInterp();
	float IntLowPassInterp = ListenerInterior.GetInteriorLPFInterp();

	if (Sound.InteriorSettings.GetInteriorStartTime() > ListenerInterior.GetInteriorStartTime())
	{
		ExtVolumeInterp = Sound.InteriorSettings.GetExteriorVolumeInterp();
		IntVolumeInterp = Sound.InteriorSettings.GetInteriorVolumeInterp();
		ExtLowPassInterp = Sound.InteriorSettings.GetExteriorLPFInterp();
		IntLowPassInterp = Sound.InteriorSettings.GetInteriorLPFInterp();
	}

	// Attenuation
	if (bAffectedByAttenuation || !bAllowSpatialization)
	{
		// Sound and listener are inside
		SourceInteriorVolume = FMath::Lerp(SourceInteriorVolume, 1.0f, IntVolumeInterp);
	}
	else
	{
		if (bUsingWorldSettings)
		{
			// Sound is outside, listener is inside
			SourceInteriorVolume = FMath::Lerp(SourceInteriorVolume, ListenerInterior.GetExteriorVolume(), ExtVolumeInterp);
		}
		else
		{
			// Sound is inside, listener is outside - Use the sound's interior volume multiplied with the listeners exterior volume
			float SoundInteriorValue = FMath::Lerp(SourceInteriorVolume, Sound.InteriorSettings.GetInteriorVolume(), IntVolumeInterp);
			float ListenerInteriorValue = FMath::Lerp(SourceInteriorVolume, ListenerInterior.GetExteriorVolume(), ExtVolumeInterp);
			SourceInteriorVolume = SoundInteriorValue * ListenerInteriorValue;
		}
	}

	// Filter
	if (bAffectedByFilter || !bAllowSpatialization)
	{
		// Sound and listener are inside
		SourceInteriorLPF = FMath::Lerp(SourceInteriorLPF, MAX_FILTER_FREQUENCY, IntLowPassInterp);
	}
	else
	{
		if (bUsingWorldSettings)
		{
			// Sound is outside, listener is inside
			SourceInteriorLPF = FMath::Lerp(SourceInteriorLPF, ListenerInterior.GetExteriorLPF(), ExtLowPassInterp);
		}
		else
		{
			// Sound is inside, listener is outside - Current interior LPF is the lesser of the Sound and Listener's LPFs
			float SoundLPFValue = FMath::Lerp(SourceInteriorLPF, Sound.InteriorSettings.GetInteriorLPF(), IntLowPassInterp);
			float ListenerLPFValue = FMath::Lerp(SourceInteriorLPF, ListenerInterior.GetExteriorLPF(), ExtLowPassInterp);
			SourceInteriorLPF = FMath::Min(SoundLPFValue, ListenerLPFValue);
		}
	}
}


constexpr TCHAR FProxyVolumeMutator::MutatorBaseName[];

FProxyVolumeMutator::FProxyVolumeMutator()
{
	MutatorName = MutatorBaseName;
}

void FProxyVolumeMutator::Apply(FInteriorSettings& InteriorSettings) const
{
	InteriorSettings.bIsWorldSettings = false;
}

bool FProxyVolumeMutator::HasPayloadType(PayloadFlags InType) const
{
	return (PayloadType & InType) != PayloadFlags::AGCP_None;
}

UAudioGameplayVolumeMutator::UAudioGameplayVolumeMutator(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UAudioGameplayVolumeMutator::SetPriority(int32 InPriority)
{
	if (Priority != InPriority)
	{
		Priority = InPriority;
		NotifyDataChanged();
	}
}

TSharedPtr<FProxyVolumeMutator> UAudioGameplayVolumeMutator::CreateMutator() const
{
	TSharedPtr<FProxyVolumeMutator> ProxyMutator = FactoryMutator();
	if (ProxyMutator.IsValid())
	{
		CopyAudioDataToMutatorBase(ProxyMutator);
	}

	return ProxyMutator;
}

#if WITH_EDITOR
void UAudioGameplayVolumeMutator::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		NotifyDataChanged();
	}
}
#endif // WITH_EDITOR

void UAudioGameplayVolumeMutator::Enable()
{
	Super::Enable();
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> UAudioGameplayVolumeMutator::FactoryMutator() const
{
	return TSharedPtr<FProxyVolumeMutator>();
}

void UAudioGameplayVolumeMutator::NotifyDataChanged() const
{
	if (IsActive())
	{
		TInlineComponentArray<UAudioGameplayVolumeComponent*> VolumeComponents(GetOwner());
		for (UAudioGameplayVolumeComponent* VolumeComponent : VolumeComponents)
		{
			if (VolumeComponent != nullptr)
			{
				VolumeComponent->OnComponentDataChanged();
			}
		}
	}
}

void UAudioGameplayVolumeMutator::CopyAudioDataToMutatorBase(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	check(Mutator.IsValid());
	Mutator->PayloadType = PayloadType;
	Mutator->Priority = Priority;

	CopyAudioDataToMutator(Mutator);
}

