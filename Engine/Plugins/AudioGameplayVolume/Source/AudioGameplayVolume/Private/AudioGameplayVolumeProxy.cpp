// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeProxy.h"
#include "AudioAnalytics.h"
#include "AudioGameplayVolumeMutator.h"
#include "AudioGameplayVolumeSubsystem.h"
#include "AudioGameplayVolumeLogs.h"
#include "AudioGameplayVolumeComponent.h"
#include "Engine/World.h"
#include "Interfaces/IAudioGameplayCondition.h"
#include "Components/PrimitiveComponent.h"

namespace AudioGameplayVolumeConsoleVariables
{
	int32 bProxyDistanceCulling = 1;
	FAutoConsoleVariableRef CVarProxyDistanceCulling(
		TEXT("au.AudioGameplayVolumes.PrimitiveProxy.DistanceCulling"),
		bProxyDistanceCulling,
		TEXT("Skips physics body queries for proxies that are not close to the listener.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

	int32 bAllowMultiplePrimitives = 1;
	FAutoConsoleVariableRef CVarAllowMultiplePrimitives (
		TEXT("au.AudioGameplayVolumes.AllowMultiplePrimitives"),
		bAllowMultiplePrimitives,
		TEXT("Allows consideration of all the primitives on an actor for intersection tests.\n")
		TEXT("NOTE: The option to turn this off will be removed in the future. Use with caution!\n")
		TEXT("0: Off, 1: On (default)"),
		ECVF_Default);

} // namespace AudioGameplayVolumeConsoleVariables

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayVolumeProxy)

namespace AudioGameplayVolumeUtils
{
	bool NeedsPhysicsQuery(UPrimitiveComponent* PrimitiveComponent, const FVector& Position)
	{
		check(PrimitiveComponent);

		if (!PrimitiveComponent->IsPhysicsStateCreated() || !PrimitiveComponent->HasValidPhysicsState())
		{
			return false;
		}

		// Temporary kill switch for distance culling
		if (AudioGameplayVolumeConsoleVariables::bProxyDistanceCulling == 0)
		{
			return true;
		}

		// Early distance culling
		return PrimitiveComponent->Bounds.GetBox().IsInsideOrOn(Position);
	}

} // namespace AudioGameplayVolumeUtils

bool UAudioGameplayVolumeProxy::ContainsPosition(const FVector& Position) const
{ 
	return false;
}

void UAudioGameplayVolumeProxy::InitFromComponent(const UAudioGameplayVolumeComponent* Component)
{
	if (!Component || !Component->GetWorld())
	{
		UE_LOGF(AudioGameplayVolumeLog, Verbose, "AudioGameplayVolumeProxy - Attempted Init from invalid volume component!");
		return;
	}
	
#if !UE_BUILD_SHIPPING
	AGVOwnerKey = Component->GetOwner();
#endif

	VolumeID = Component->GetUniqueID();
	WorldID = Component->GetWorld()->GetUniqueID();

	PayloadType = PayloadFlags::AGCP_None;
	ProxyVolumeMutators.Reset();

	TInlineComponentArray<UAudioGameplayVolumeMutator*> Components(Component->GetOwner());
	for (UAudioGameplayVolumeMutator* Comp : Components)
	{
		if (!Comp || !Comp->IsActive())
		{
			continue;
		}

		TSharedPtr<FProxyVolumeMutator> NewMutator = Comp->CreateMutator();
		if (NewMutator.IsValid())
		{
			NewMutator->VolumeID = VolumeID;
			NewMutator->WorldID = WorldID;

			AddPayloadType(NewMutator->PayloadType);
			ProxyVolumeMutators.Emplace(NewMutator);
		}
	}

	Audio::Analytics::RecordEvent_Usage(TEXT("AudioGameplayVolume.InitializedFromComponent"));
}

void UAudioGameplayVolumeProxy::CollectMutators(ProxyMutatorList& OutMutators) const
{
	check(IsInAudioThread());
	OutMutators.Append(ProxyVolumeMutators);
}


void UAudioGameplayVolumeProxy::AddPayloadType(PayloadFlags InType)
{
	PayloadType |= InType;
}

bool UAudioGameplayVolumeProxy::HasPayloadType(PayloadFlags InType) const
{
	return (PayloadType & InType) != PayloadFlags::AGCP_None;
}

uint32 UAudioGameplayVolumeProxy::GetVolumeID() const
{ 
	return VolumeID;
}

uint32 UAudioGameplayVolumeProxy::GetWorldID() const
{
	return WorldID;
}

#if !UE_BUILD_SHIPPING
FName UAudioGameplayVolumeProxy::GetOwnerName() const
{
	return GetFNameSafe(AGVOwnerKey.ResolveObjectPtr());
}
#endif

bool UAGVPrimitiveComponentProxy::ContainsPosition(const FVector& Position) const
{
	SCOPED_NAMED_EVENT(UAGVPrimitiveComponentProxy_ContainsPosition, FColor::Blue);

	bool ContainsPosition = false;

	for (const TObjectPtr<UPrimitiveComponent>& PrimitiveComponent : Primitives)
	{
		if (!PrimitiveComponent)
		{
			continue;
		}

		const FBodyInstance* BodyInstance = PrimitiveComponent->GetBodyInstance();
		if (!BodyInstance)
		{
			continue;
		}

		if (AudioGameplayVolumeUtils::NeedsPhysicsQuery(PrimitiveComponent, Position))
		{
			float DistanceSquared = 0.f;
			FVector PointOnBody = FVector::ZeroVector;
			if (BodyInstance->GetSquaredDistanceToBody(Position, DistanceSquared, PointOnBody) && FMath::IsNearlyZero(DistanceSquared))
			{
				ContainsPosition = true;
				break;
			}
		}
	}

	return ContainsPosition;
}

void UAGVPrimitiveComponentProxy::InitFromComponent(const UAudioGameplayVolumeComponent* Component)
{
	Super::InitFromComponent(Component);
	Primitives.Reset();

	if (Component)
	{
		TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents(Component->GetOwner());
		int32 PrimitiveCount = PrimitiveComponents.Num();

		if (PrimitiveCount > 0)
		{
			// Using this console variable as a temporary rollback to give us time to respond in live development.
			// Not expecting this rollback to live long as the (original) case it's protecting was non-deterministic
			if (!AudioGameplayVolumeConsoleVariables::bAllowMultiplePrimitives)
			{
				Primitives.Emplace(PrimitiveComponents[0]);
			}
			else
			{
				Primitives.Reserve(PrimitiveCount);
				Algo::Copy(PrimitiveComponents, Primitives);
			}
		}
	}
}

bool UAGVConditionProxy::ContainsPosition(const FVector& Position) const
{
	SCOPED_NAMED_EVENT(UAGVConditionProxy_ContainsPosition, FColor::Blue);

	if (ObjectPtr && ObjectPtr->Implements<UAudioGameplayCondition>())
	{
		return IAudioGameplayCondition::Execute_ConditionMet(ObjectPtr)
			|| IAudioGameplayCondition::Execute_ConditionMet_Position(ObjectPtr, Position);
	}

	return false;
}

void UAGVConditionProxy::InitFromComponent(const UAudioGameplayVolumeComponent* Component)
{
	Super::InitFromComponent(Component);

	AActor* OwnerActor = Component ? Component->GetOwner() : nullptr;
	if (OwnerActor)
	{
		if (OwnerActor->Implements<UAudioGameplayCondition>())
		{
			ObjectPtr = OwnerActor;
		}
		else
		{
			TInlineComponentArray<UActorComponent*> AllComponents(OwnerActor);

			for (UActorComponent* ActorComponent : AllComponents)
			{
				if (ActorComponent && ActorComponent->Implements<UAudioGameplayCondition>())
				{
					ObjectPtr = ActorComponent;
					break;
				}
			}
		}
	}
}
