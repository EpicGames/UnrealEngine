// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeSubsystem.h"
#include "ActiveSound.h"
#include "Algo/Transform.h"
#include "AudioGameplayVolumeLogs.h"
#include "AudioGameplayVolumeComponent.h"
#include "AudioGameplayVolumeProxy.h"
#include "AudioGameplayVolumeMutator.h"
#include "AudioDevice.h"
#include "Engine/World.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/ScopeLock.h"
#include "Stats/Stats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AudioGameplayVolumeSubsystem)

namespace AudioGameplayVolumeConsoleVariables
{
	int32 bEnabled = 1;
	FAutoConsoleVariableRef CVarEnabled(
		TEXT("au.AudioGameplayVolumes.Enabled"),
		bEnabled,
		TEXT("Toggles the Audio Gameplay Volume System on or off.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

	int32 bUpdateListeners = 1;
	FAutoConsoleVariableRef CVarUpdateListeners(
		TEXT("au.AudioGameplayVolumes.Listeners.AllowUpdate"),
		bUpdateListeners,
		TEXT("Allows updating of listeners.\n0: Disable, 1: Enable (default)"),
		ECVF_Default);

	float MinUpdateRate = 0.016f;
	float UpdateRate = 0.05f;
	FAutoConsoleVariableRef CVarUpdateInterval(
		TEXT("au.AudioGameplayVolume.UpdateRate"),
		UpdateRate,
		TEXT("How frequently we check for listener changes with respect to audio gameplay volumes, in seconds."),
		ECVF_Default);

	float UpdateRateJitterDelta = 0.025f;
	FAutoConsoleVariableRef CVarUpdateRateJitterDelta(
		TEXT("au.AudioGameplayVolume.UpdateRate.JitterDelta"),
		UpdateRateJitterDelta,
		TEXT("A random delta to add to update rate to avoid performance heartbeats."),
		ECVF_Default);

	int32 bAudioThreadCmdRollback = 0;
	FAutoConsoleVariableRef CVarAudioThreadCmdRollback(
		TEXT("au.AudioGameplayVolumes.AudioThreadCmdRollback"),
		bAudioThreadCmdRollback,
		TEXT("When non-zero, uses old code to rollback late thread command change."),
		ECVF_Default);
	
	int32 bPreventDuplicateSubmixSends = 0;
	FAutoConsoleVariableRef CVarPreventDuplicateSubmixSends(
		TEXT("au.AudioGameplayVolumes.PreventDuplicateSubmixSends"),
		bPreventDuplicateSubmixSends,
		TEXT("When non-zero, will not add a Submix Sends if one already is there sending to the same Submix."),
		ECVF_Default);

#if !UE_BUILD_SHIPPING
	float MicroSecsBeforeWarnContainsPositionTooLong = 40.;
	FAutoConsoleVariableRef CVarMicroSecsBeforeWarnContainsPositionTooLong(
		TEXT("au.AudioGameplayVolumes.MicroSecsBeforeWarnContainsPositionTooLong"),
		MicroSecsBeforeWarnContainsPositionTooLong,
		TEXT("Warns whenever a ContainsPosition test on an AGV would take more than this time, logging the name of the expensive volume."),
		ECVF_Default);
#endif
} // namespace AudioGameplayVolumeConsoleVariables

void FAudioGameplayActiveSoundInfo::Update(double ListenerInteriorStartTime)
{
	// If the interior settings have changed for the listener or the sound, we need to set
	// new interpolation targets.  However, we need to 'initialize' the sounds interior start time to match
	// the listener's interior start time, to prevent new sounds from interpolating until they change interior settings.
	if (FMath::IsNearlyZero(LastUpdateTime))
	{
		InteriorSettings.SetInteriorStartTime(ListenerInteriorStartTime);
	}

	if (LastUpdateTime < ListenerInteriorStartTime || LastUpdateTime < InteriorSettings.GetInteriorStartTime())
	{
		SourceInteriorVolume = CurrentInteriorVolume;
		SourceInteriorLPF = CurrentInteriorLPF;
		LastUpdateTime = FApp::GetCurrentTime();
	}

	InteriorSettings.UpdateInteriorValues();
}

void FAudioProxyMutatorSearchObject::SearchVolumes(const TArray<UAudioGameplayVolumeProxy*>& ProxyVolumes, FAudioProxyMutatorSearchResult& OutResult)
{
	check(IsInAudioThread());
	SCOPED_NAMED_EVENT(FAudioProxyMutatorSearchObject_SearchVolumes, FColor::Blue);

	OutResult.Reset();

	for (const UAudioGameplayVolumeProxy* ProxyVolume : ProxyVolumes)
	{
		if (!ProxyVolume || ProxyVolume->GetWorldID() != WorldID)
		{
			continue;
		}

		if (bFilterPayload && !ProxyVolume->HasPayloadType(PayloadType))
		{
			continue;
		}
		
		{
#if !UE_BUILD_SHIPPING
			const uint64 StartTime = FPlatformTime::Cycles64();
			ON_SCOPE_EXIT{
				const double DurationMicroSecs = FTimespan(FPlatformTime::Cycles64() - StartTime).GetTotalMicroseconds();
				if (DurationMicroSecs >= AudioGameplayVolumeConsoleVariables::MicroSecsBeforeWarnContainsPositionTooLong)
				{
					// Some AGV proxies might just have costly ContainsPosition() checks, or Brush-based volumes might just have a shape that's too 
					// complex. This allows to find outliers.
					// Note: With WorldPartition enabled and its UAID_ actor names, you can still search your level in the Editor
					//       Outliner for that string and the volume actor will show up.
					UE_LOGFMT(AudioGameplayVolumeLog, Verbose, "{AGV} ContainsPosition() took {Duration} µs", ProxyVolume->GetOwnerName(), DurationMicroSecs);
				}
			};
#endif

			if (!ProxyVolume->ContainsPosition(Location))
			{
				continue;
			}
		}

		if (bCollectMutators)
		{
			ProxyVolume->CollectMutators(OutResult.MatchingMutators);
		}
		OutResult.VolumeSet.Add(ProxyVolume->GetVolumeID());
	}

	// Use 'world settings' as a starting point
	if (!bAffectedByLegacySystem && AudioDeviceHandle.IsValid())
	{
		AudioDeviceHandle->GetDefaultAudioSettings(WorldID, OutResult.ReverbSettings, OutResult.InteriorSettings);
	}

	// Sort our mutators and apply interior settings
	if (bCollectMutators)
	{
		auto SortPredicate = [](TSharedPtr<FProxyVolumeMutator>& A, TSharedPtr<FProxyVolumeMutator>& B)
		{
			check(A.IsValid() && B.IsValid());

			if (A->MutatorName.IsEqual(B->MutatorName))
			{
				return A->Priority > B->Priority;
			}

			return A->MutatorName.FastLess(B->MutatorName);
		};

		// Remove invalid mutators and sort
		OutResult.MatchingMutators.RemoveAllSwap([](TSharedPtr<FProxyVolumeMutator>& ProxyMutator) { return !ProxyMutator.IsValid(); });
		Algo::Sort(OutResult.MatchingMutators, SortPredicate);

		FName MutatorGroup;
		int32 CurrentPriority = INDEX_NONE;

		// Apply mutators to our interior settings.
		// Mutators are sorted by type (name) and descending priority, so we can quickly apply the highest priority mutators.
		for (TSharedPtr<FProxyVolumeMutator>& Mutator : OutResult.MatchingMutators)
		{
			// If we encounter a new name, it's a new type - with the highest priority for that group
			if (!Mutator->MutatorName.IsEqual(MutatorGroup))
			{
				MutatorGroup = Mutator->MutatorName;
				CurrentPriority = Mutator->Priority;
			}

			// Lower priority mutators are ignored, we need to advance to the next named group
			if (Mutator->Priority < CurrentPriority)
			{
				continue;
			}

			Mutator->Apply(OutResult.InteriorSettings);
		}
	}
}

void FAudioGameplayVolumeProxyInfo::Update(const TArray<FAudioGameplayVolumeListener>& VolumeListeners, FAudioGameplayProxyUpdateResult& OutResult)
{
	check(IsInAudioThread());

	PreviousProxies = MoveTemp(CurrentProxies);
	for (const int32 ListenerIndex : ListenerIndexes)
	{
		if (ListenerIndex <= VolumeListeners.Num())
		{
			CurrentProxies.Append(VolumeListeners[ListenerIndex].GetCurrentProxies());
		}
	}

	ListenerIndexes.Reset();

	// We've entered proxies that are in the current list, but not in previous
	OutResult.EnteredProxies = CurrentProxies.Difference(PreviousProxies);

	// We've exited proxies that are in the previous list, but not in current
	OutResult.ExitedProxies = PreviousProxies.Difference(CurrentProxies);
}

void FAudioGameplayVolumeProxyInfo::AddListenerIndex(int32 ListenerIndex)
{
	ListenerIndexes.Emplace(ListenerIndex);
}

bool FAudioGameplayVolumeProxyInfo::IsVolumeInCurrentList(uint32 VolumeID) const
{
	return CurrentProxies.Contains(VolumeID);
}

bool UAudioGameplayVolumeSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	return !IsRunningDedicatedServer();
}

void UAudioGameplayVolumeSubsystem::Deinitialize()
{
	FScopeLock Lock(&AGVComponentsCriticalSection);
	AGVComponents.Reset();
}

void UAudioGameplayVolumeSubsystem::Update()
{
	check(IsInAudioThread());
	
	// We track and check our previous proxy count to ensure we get one update after all of the volumes have been removed
	if (AudioGameplayVolumeConsoleVariables::bEnabled == 0 || (KnownProxyIDs.Num() == 0 && PreviousProxyCount == 0))
	{
		return;
	}

	// limit updates for perf - not necessary to update every tick
	const float DeltaTime = FApp::GetDeltaTime();
	TimeSinceUpdate += DeltaTime;
	if (TimeSinceUpdate < NextUpdateDeltaTime)
	{
		return;
	}

	const float JitterDelta = FMath::RandRange(0.f, AudioGameplayVolumeConsoleVariables::UpdateRateJitterDelta);
	NextUpdateDeltaTime = FMath::Max(AudioGameplayVolumeConsoleVariables::UpdateRate + JitterDelta, AudioGameplayVolumeConsoleVariables::MinUpdateRate);
	TimeSinceUpdate = 0.f;

	if (bHasStaleProxy)
	{
		FAudioDeviceHandle AudioDeviceHandle = GetAudioDeviceHandle();
		if (AudioDeviceHandle.IsValid())
		{
			AudioDeviceHandle->InvalidateCachedInteriorVolumes();
			bHasStaleProxy = false;
		}
	}

	if (AudioGameplayVolumeConsoleVariables::bUpdateListeners != 0)
	{
		UpdateFromListeners();
	}

	PreviousProxyCount = KnownProxyIDs.Num();
}

void UAudioGameplayVolumeSubsystem::GatherInteriorData(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams)
{
	if (AudioGameplayVolumeConsoleVariables::bEnabled == 0)
	{
		return;
	}

	check(IsInAudioThread());

	FAudioProxyMutatorSearchObject MutatorSearch;
	MutatorSearch.WorldID = ActiveSound.GetWorldID();
	MutatorSearch.Location = ParseParams.Transform.GetTranslation();
	MutatorSearch.PayloadType = AudioGameplay::EComponentPayload::AGCP_ActiveSound;
	MutatorSearch.AudioDeviceHandle = GetAudioDeviceHandle();
	MutatorSearch.bAffectedByLegacySystem = (ActiveSound.AudioVolumeID != 0);

	GenerateVolumeProxyList();
	FAudioProxyMutatorSearchResult Result;
	MutatorSearch.SearchVolumes(TransientProxyList, Result);

	// Save info about this active sound for application.
	FAudioGameplayActiveSoundInfo& ActiveSoundInfo = ActiveSoundData.FindOrAdd(ActiveSound.GetInstanceID());
	ActiveSoundInfo.CurrentMutators = MoveTemp(Result.MatchingMutators);
	ActiveSoundInfo.InteriorSettings.Apply(Result.InteriorSettings);

	// We need to drop references, but we can at least prevent some allocs from using a member variable
	TransientProxyList.Reset();
}

void UAudioGameplayVolumeSubsystem::ApplyInteriorSettings(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams)
{
	if (AudioGameplayVolumeConsoleVariables::bEnabled == 0)
	{
		return;
	}

	check(IsInAudioThread());

	int32 ListenerIndex = ActiveSound.GetClosestListenerIndex();
	if (ListenerIndex >= AGVListeners.Num())
	{
		return;
	}

	// Legacy audio volumes is affecting this sound, do not update
	const FAudioGameplayVolumeListener& Listener = AGVListeners[ListenerIndex];
	if(ActiveSound.AudioVolumeID != 0 || Listener.GetAffectedByLegacySystem())
	{
		return;
	}

	FAudioGameplayActiveSoundInfo* ActiveSoundInfo = ActiveSoundData.Find(ActiveSound.GetInstanceID());
	if (!ActiveSoundInfo)
	{
		return;
	}

	ActiveSoundInfo->Update(Listener.GetInteriorSettings().GetInteriorStartTime());

	uint32 WorldID = ActiveSound.GetWorldID();
	FAudioProxyActiveSoundParams Params(*ActiveSoundInfo, Listener);
	Params.bAllowSpatialization = ActiveSound.bAllowSpatialization;

	// Apply mutators to our active sound params
	// Mutators are sorted by type (name) and descending priority, so we can quickly apply the highest priority mutators.
	FName MutatorGroup;
	int32 CurrentPriority = INDEX_NONE;
	for (const TSharedPtr<FProxyVolumeMutator>& Mutator : ActiveSoundInfo->CurrentMutators)
	{
		if (!Mutator.IsValid())
		{
			continue;
		}

		// If we encounter a new name, it's a new type - with the highest priority for that group
		if (!Mutator->MutatorName.IsEqual(MutatorGroup))
		{
			MutatorGroup = Mutator->MutatorName;
			CurrentPriority = Mutator->Priority;
		}

		// Lower priority mutators are ignored, we need to advance to the next named group
		if (Mutator->Priority < CurrentPriority)
		{
			continue;
		}

		Params.bListenerInVolume = IsAnyListenerInVolume(WorldID, Mutator->VolumeID);
		Mutator->Apply(Params);
	}

	// Update interior values
	Params.UpdateInteriorValues();
	ActiveSoundInfo->CurrentInteriorVolume = Params.SourceInteriorVolume;
	ActiveSoundInfo->CurrentInteriorLPF = Params.SourceInteriorLPF;

	// Apply Submix Sends to our parse param
	if (AudioGameplayVolumeConsoleVariables::bPreventDuplicateSubmixSends)
	{
		// This opt-in behavior prevents the AudioGameplayVolumeSubsystem from clobbering existing Submix Sends
		// that may already exist on the asset we're interacting with.
		// Reason: If a sound has two Submix Sends to the same Submix, the current observed behavior is that only the last send is respected.
		// Example: A designer explicitly dialed in a Send to Submix A at 0.5f
		// CVAR OFF: this subsystem will clobber that send
		// CVAR  ON: this subsystem will preserve that design decision and no-op for that particular send
		bool bSendAlreadyExists = false;
		for (const FSoundSubmixSendInfo& SendA : Params.SoundSubmixSends)
		{
			for (const FSoundSubmixSendInfo& SendB : ParseParams.SoundSubmixSends)
			{
				if (SendA.SoundSubmix == SendB.SoundSubmix)
				{
					bSendAlreadyExists = true;
					break;
				}
			}
		}

		if (!bSendAlreadyExists)
		{
			ParseParams.SoundSubmixSends.Append(Params.SoundSubmixSends);
		}
	}
	else
	{
		ParseParams.SoundSubmixSends.Append(Params.SoundSubmixSends);
	}

	// Apply to the rest of our parse params
	ParseParams.InteriorVolumeMultiplier = ActiveSoundInfo->CurrentInteriorVolume;
	ParseParams.AmbientZoneFilterFrequency = ActiveSoundInfo->CurrentInteriorLPF;
}

void UAudioGameplayVolumeSubsystem::NotifyActiveSoundDeleting(const FActiveSound& ActiveSound)
{
	check(IsInAudioThread());
	if (ActiveSound.bApplyInteriorVolumes && ActiveSoundData.Num() > 0)
	{
		ActiveSoundData.Remove(ActiveSound.GetInstanceID());
	}
}

void UAudioGameplayVolumeSubsystem::AddVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent)
{
	if (!VolumeComponent)
	{
		UE_LOGF(AudioGameplayVolumeLog, Verbose, "AudioGameplayVolumeSubsystem - Attempting to add invalid volume component!");
		return;
	}
	
	if (!DoesSupportWorld(VolumeComponent->GetWorld()))
	{
		return;
	}

	const uint32 ComponentID = VolumeComponent->GetUniqueID();
	UAudioGameplayVolumeProxy* VolumeProxy = VolumeComponent->GetProxy();
	if (VolumeProxy && !AGVComponents.Contains(ComponentID))
	{
		VolumeProxy->InitFromComponent(VolumeComponent);
		{
			FScopeLock Lock(&AGVComponentsCriticalSection);
			AGVComponents.Emplace(ComponentID, VolumeComponent);
		}
		const uint32 WorldID = VolumeProxy->GetWorldID();

		UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "AudioGameplayVolumeComponent %ls [%08x] added", *VolumeComponent->GetFName().ToString(), ComponentID);

		// Copy representation of volume to audio thread
		Audio::FDeviceId CurrentDeviceId = GetAudioDeviceId();
		FAudioThread::RunCommandOnAudioThread([CurrentDeviceId, ComponentID, WorldID]()
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				FAudioDeviceHandle DeviceHandle = AudioDeviceManager->GetAudioDevice(CurrentDeviceId);
				if (DeviceHandle.IsValid())
				{
					if (UAudioGameplayVolumeSubsystem* AGVSubsystem = DeviceHandle->GetSubsystem<UAudioGameplayVolumeSubsystem>())
					{
						AGVSubsystem->AddProxy(ComponentID, WorldID);
					}
				}
			}
		});
	}
}

void UAudioGameplayVolumeSubsystem::UpdateVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent)
{
	if (!VolumeComponent || !VolumeComponent->GetWorld())
	{
		UE_LOGF(AudioGameplayVolumeLog, Verbose, "AudioGameplayVolumeSubsystem - Attempting to update invalid volume component!");
		return;
	}

	const uint32 ComponentID = VolumeComponent->GetUniqueID();
	UAudioGameplayVolumeProxy* VolumeProxy = VolumeComponent->GetProxy();
	if (VolumeProxy && AGVComponents.Contains(ComponentID))
	{
		VolumeProxy->InitFromComponent(VolumeComponent);

		UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "AudioGameplayVolumeComponent %ls [%08x] updated", *VolumeComponent->GetFName().ToString(), ComponentID);

		// Update representation of volume on audio thread
		Audio::FDeviceId CurrentDeviceId = GetAudioDeviceId();
		FAudioThread::RunCommandOnAudioThread([CurrentDeviceId, ComponentID]()
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				FAudioDeviceHandle DeviceHandle = AudioDeviceManager->GetAudioDevice(CurrentDeviceId);
				if (DeviceHandle.IsValid())
				{
					if (UAudioGameplayVolumeSubsystem* AGVSubsystem = DeviceHandle->GetSubsystem<UAudioGameplayVolumeSubsystem>())
					{
						AGVSubsystem->UpdateProxy(ComponentID);
					}
				}
			}
		});
	}
}

void UAudioGameplayVolumeSubsystem::RemoveVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent)
{
	if (!VolumeComponent || !VolumeComponent->GetWorld())
	{
		UE_LOGF(AudioGameplayVolumeLog, Verbose, "AudioGameplayVolumeSubsystem - Attempting to remove invalid volume component!");
		return;
	}

	const uint32 ComponentID = VolumeComponent->GetUniqueID();
	if (AGVComponents.Contains(ComponentID))
	{
		FScopeLock Lock(&AGVComponentsCriticalSection);
		AGVComponents.Remove(ComponentID);
		UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "AudioGameplayVolumeComponent %ls [%08x] removed", *VolumeComponent->GetFName().ToString(), ComponentID);
	}

	// Remove representation of volume from audio thread
	if (AudioGameplayVolumeConsoleVariables::bAudioThreadCmdRollback)
	{
		TWeakObjectPtr<UAudioGameplayVolumeSubsystem> WeakThis(this);
		FAudioThread::RunCommandOnAudioThread([WeakThis, ComponentID]()
		{
			if (WeakThis.IsValid())
			{
				WeakThis->RemoveProxy(ComponentID);
			}
		});
	}
	else
	{
		Audio::FDeviceId CurrentDeviceId = GetAudioDeviceId();
		FAudioThread::RunCommandOnAudioThread([CurrentDeviceId, ComponentID]()
		{
			if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
			{
				FAudioDeviceHandle DeviceHandle = AudioDeviceManager->GetAudioDevice(CurrentDeviceId);
				if (DeviceHandle.IsValid())
				{
					if (UAudioGameplayVolumeSubsystem* AGVSubsystem = DeviceHandle->GetSubsystem<UAudioGameplayVolumeSubsystem>())
					{
						AGVSubsystem->RemoveProxy(ComponentID);
					}
				}
			}
		});
	}
}

bool UAudioGameplayVolumeSubsystem::DoesSupportWorld(UWorld* World) const
{
	if (World && (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE))
	{
		return true;
	}
	return false;
}

bool UAudioGameplayVolumeSubsystem::AddProxy(uint32 AudioGameplayVolumeID, uint32 WorldID)
{
	check(IsInAudioThread());

	const UAudioGameplayVolumeComponent* AGVComponent;
	{
		FScopeLock Lock(&AGVComponentsCriticalSection);
		AGVComponent = AGVComponents.FindRef(AudioGameplayVolumeID);
	}

	// Make sure the component is in the map and non-null
	if (AGVComponent == nullptr)
	{
		return false;
	}

	// Make sure our proxy is valid
	if (AGVComponent->GetProxy() == nullptr)
	{
		return false;
	}

	if (KnownProxyIDs.Contains(AudioGameplayVolumeID))
	{
		UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "Attempting to add Proxy [%08x] multiple times", AudioGameplayVolumeID);
		return false;
	}

	KnownProxyIDs.Add(AudioGameplayVolumeID);
	WorldProxyLists.FindOrAdd(WorldID);
	bHasStaleProxy = true;

	UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "Proxy [%08x] added", AudioGameplayVolumeID);
	return true;
}

bool UAudioGameplayVolumeSubsystem::UpdateProxy(uint32 AudioGameplayVolumeID)
{
	check(IsInAudioThread());

	if (KnownProxyIDs.Contains(AudioGameplayVolumeID))
	{
		bHasStaleProxy = true;
		return true;
	}

	return false;
}

bool UAudioGameplayVolumeSubsystem::RemoveProxy(uint32 AudioGameplayVolumeID)
{
	check(IsInAudioThread());

	const int32 NumRemoved = KnownProxyIDs.Remove(AudioGameplayVolumeID);
	if (NumRemoved > 0)
	{
		bHasStaleProxy = true;
		UE_LOGF(AudioGameplayVolumeLog, VeryVerbose, "Removed %d Proxies with Id [%08x]", NumRemoved, AudioGameplayVolumeID);
		return true;
	}

	return false;
}

void UAudioGameplayVolumeSubsystem::GenerateVolumeProxyList()
{
	TransientProxyList.Reset(256);
	
	FScopeLock Lock(&AGVComponentsCriticalSection);
	Algo::Transform(AGVComponents, TransientProxyList, [](const TPair<uint32, TObjectPtr<const UAudioGameplayVolumeComponent>>& Pair)
	{
		return Pair.Value ? Pair.Value->GetProxy() : nullptr;
	});
}

bool UAudioGameplayVolumeSubsystem::IsAnyListenerInVolume(uint32 WorldID, uint32 VolumeID) const
{
	// We test this by checking to see if the volume id provided is in our world's current proxy list
	if (const FAudioGameplayVolumeProxyInfo* ProxyInfo = WorldProxyLists.Find(WorldID))
	{
		return ProxyInfo->IsVolumeInCurrentList(VolumeID);
	}
	return false;
}

void UAudioGameplayVolumeSubsystem::UpdateComponentsFromProxyInfo(const FAudioGameplayProxyUpdateResult& ProxyResults) const
{
	check(IsInGameThread());
	for (uint32 VolumeID : ProxyResults.EnteredProxies)
	{
		if (!AGVComponents.Contains(VolumeID))
		{
			continue;
		}

		if (const UAudioGameplayVolumeComponent* ProxyComponent = AGVComponents[VolumeID])
		{
			ProxyComponent->EnterProxy();
		}
	}

	for (uint32 VolumeID : ProxyResults.ExitedProxies)
	{
		if (!AGVComponents.Contains(VolumeID))
		{
			continue;
		}

		if (const UAudioGameplayVolumeComponent* ProxyComponent = AGVComponents[VolumeID])
		{
			ProxyComponent->ExitProxy();
		}
	}
}

void UAudioGameplayVolumeSubsystem::UpdateFromListeners()
{
	check(IsInAudioThread());
	SCOPED_NAMED_EVENT(UAudioGameplayVolumeSubsystem_UpdateFromListeners, FColor::Blue);

	FAudioDeviceHandle DeviceHandle = GetAudioDeviceHandle();
	check(DeviceHandle.IsValid());
	const uint32 AudioDeviceID = DeviceHandle.GetDeviceID();

	const int32 ListenerCount = DeviceHandle->GetListeners().Num();
	AGVListeners.SetNum(ListenerCount, EAllowShrinking::No);
	constexpr bool bAllowAttenuationOverride = true;

	// We have to search twice, as we only care about mutators that affect listeners, but we care about ALL proxyVolumes we're a part of
	FAudioProxyMutatorSearchObject ProxySearch;
	ProxySearch.AudioDeviceHandle = DeviceHandle;
	ProxySearch.PayloadType = AudioGameplay::EComponentPayload::AGCP_Listener;

	FAudioProxyMutatorSearchResult Result;
	TSet<uint32> TempVolumeSet;
	FTransform ListenerTransform;
	
	GenerateVolumeProxyList();

	// Grabbing the listeners directly here should be removed when possible - done out of necessity due to the legacy audio volume system
	const TArray<FListener>& AudioListeners = DeviceHandle->GetListeners();

	// Update our audio gameplay volume listeners
	for (int32 i = 0; i < ListenerCount; ++i)
	{
		const bool bValidAudioListener = DeviceHandle->GetListenerTransform(i, ListenerTransform);
		if (bValidAudioListener && !ListenerTransform.Equals(FTransform::Identity))
		{
			// Fill location and worldID
			if (DeviceHandle->GetListenerWorldID(i, ProxySearch.WorldID) && DeviceHandle->GetListenerPosition(i, ProxySearch.Location, bAllowAttenuationOverride))
			{
				// Find only the proxy volumes we're inside of, regardless of payload type
				ProxySearch.bFilterPayload = false;
				ProxySearch.bCollectMutators = false;
				ProxySearch.bGetDefaultAudioSettings = false;
				ProxySearch.SearchVolumes(TransientProxyList, Result);

				// Hold on to these
				Swap(TempVolumeSet, Result.VolumeSet);

				// Second search - this time for mutators
				ProxySearch.bFilterPayload = true;
				ProxySearch.bCollectMutators = true;
				ProxySearch.bGetDefaultAudioSettings = true;
				ProxySearch.SearchVolumes(TransientProxyList, Result);

				if (i < AudioListeners.Num())
				{
					AGVListeners[i].SetAffectedByLegacySystem(AudioListeners[i].AudioVolumeID != 0);
				}

				// Reassign the set of all proxy volumes (regardless of payload type)
				Swap(TempVolumeSet, Result.VolumeSet);
				AGVListeners[i].Update(Result, ProxySearch.Location, AudioDeviceID);

				if (FAudioGameplayVolumeProxyInfo* ProxyInfo = WorldProxyLists.Find(ProxySearch.WorldID))
				{
					ProxyInfo->AddListenerIndex(i);
				}

				continue;
			}
		}

		// Listener is invalid or uninitialized
		Result.Reset();
		AGVListeners[i].Update(Result, FVector::ZeroVector, AudioDeviceID);
	}

	// We need to drop references, but we can at least prevent some allocs from using a member variable
	TransientProxyList.Reset();
	
	// For all our worlds, update which proxies each listener is in and determine if there was a change
	FAudioGameplayProxyUpdateResult ProxyUpdateResult;
	TArray<FAudioGameplayProxyUpdateResult> UpdateResultsForProxyWithPresenceChanges;
	for (TPair<uint32, FAudioGameplayVolumeProxyInfo>& WorldProxyInfo : WorldProxyLists)
	{
		WorldProxyInfo.Value.Update(AGVListeners, ProxyUpdateResult);
		if (!ProxyUpdateResult.EnteredProxies.IsEmpty() || !ProxyUpdateResult.ExitedProxies.IsEmpty())
		{
			UpdateResultsForProxyWithPresenceChanges.Add(ProxyUpdateResult);
		}
	}

	// If we had some changes, batch-broadcast the proxy entries/exits notifications
	if (!UpdateResultsForProxyWithPresenceChanges.IsEmpty())
	{
		FAudioThread::RunCommandOnGameThread([WeakThis = TWeakObjectPtr(this), ProxyUpdateResults = MoveTemp(UpdateResultsForProxyWithPresenceChanges)]
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_AudioCommand_UpdateComponentsFromProxyInfo);
			if (ThisClass* This = WeakThis.Get())
			{
				for (const FAudioGameplayProxyUpdateResult& ProxyUpdateResult : ProxyUpdateResults)
				{
					This->UpdateComponentsFromProxyInfo(ProxyUpdateResult);
				}
			}
		});
	}
}
