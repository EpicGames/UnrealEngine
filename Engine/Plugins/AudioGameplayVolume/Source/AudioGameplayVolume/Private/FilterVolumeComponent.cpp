// Copyright Epic Games, Inc. All Rights Reserved.

#include "FilterVolumeComponent.h"
#include "Sound/AudioVolume.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FilterVolumeComponent)

constexpr TCHAR FProxyMutator_Filter::MutatorFilterName[];

FProxyMutator_Filter::FProxyMutator_Filter()
{
	MutatorName = MutatorFilterName;
}

void FProxyMutator_Filter::Apply(FInteriorSettings& InteriorSettings) const
{
	FProxyVolumeMutator::Apply(InteriorSettings);

	InteriorSettings.ExteriorLPF = ExteriorLPF;
	InteriorSettings.ExteriorLPFTime = ExteriorLPFTime;
	InteriorSettings.InteriorLPF = InteriorLPF;
	InteriorSettings.InteriorLPFTime = InteriorLPFTime;
}

void FProxyMutator_Filter::Apply(FAudioProxyActiveSoundParams& Params) const
{
	Params.bAffectedByFilter |= Params.bListenerInVolume;
}

UFilterVolumeComponent::UFilterVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PayloadType = AudioGameplay::EComponentPayload::AGCP_ActiveSound | AudioGameplay::EComponentPayload::AGCP_Listener;
	bAutoActivate = true;
}

void UFilterVolumeComponent::SetExteriorLPF(float Volume, float InterpolateTime)
{
	ExteriorLPF = Volume;
	ExteriorLPFTime = InterpolateTime;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

void UFilterVolumeComponent::SetInteriorLPF(float Volume, float InterpolateTime)
{
	InteriorLPF = Volume;
	InteriorLPFTime = InterpolateTime;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> UFilterVolumeComponent::FactoryMutator() const
{
	return MakeShared<FProxyMutator_Filter>();
}

void UFilterVolumeComponent::CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	TSharedPtr<FProxyMutator_Filter> FilterMutator = StaticCastSharedPtr<FProxyMutator_Filter>(Mutator);
	FilterMutator->ExteriorLPF = ExteriorLPF;
	FilterMutator->ExteriorLPFTime = ExteriorLPFTime;
	FilterMutator->InteriorLPF = InteriorLPF;
	FilterMutator->InteriorLPFTime = InteriorLPFTime;
}

