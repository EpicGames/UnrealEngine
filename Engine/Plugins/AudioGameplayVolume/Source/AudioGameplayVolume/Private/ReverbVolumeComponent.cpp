// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReverbVolumeComponent.h"
#include "AudioDevice.h"
#include "AudioGameplayVolumeListener.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReverbVolumeComponent)

constexpr TCHAR FProxyMutator_Reverb::MutatorReverbName[];

FProxyMutator_Reverb::FProxyMutator_Reverb()
{
	MutatorName = MutatorReverbName;
}

void FProxyMutator_Reverb::Apply(FAudioGameplayVolumeListener& Listener) const
{
	check(IsInAudioThread());

	FAudioDeviceHandle AudioDeviceHandle = FAudioDeviceManager::Get()->GetAudioDevice(Listener.GetOwningDeviceId());
	if (AudioDeviceHandle.IsValid())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AGVActivateReverb"), STAT_AGVActivateReverb, STATGROUP_AudioThreadCommands);

		float PriorityAsFloat = static_cast<float>(Priority);
		FReverbSettings Settings = ReverbSettings;
		FAudioThread::RunCommandOnGameThread([AudioDeviceHandle, Settings, PriorityAsFloat]() mutable
		{
			if (AudioDeviceHandle.IsValid())
			{
				static FName NAME_AGVReverb(TEXT("AGVReverb"));
				AudioDeviceHandle->ActivateReverbEffect(Settings.ReverbEffect, NAME_AGVReverb, PriorityAsFloat, Settings.Volume, Settings.FadeTime);
			}
		}, GET_STATID(STAT_AGVActivateReverb));
	}
}

void FProxyMutator_Reverb::Remove(FAudioGameplayVolumeListener& Listener) const
{
	check(IsInAudioThread());

	FAudioDeviceHandle AudioDeviceHandle = FAudioDeviceManager::Get()->GetAudioDevice(Listener.GetOwningDeviceId());
	if (AudioDeviceHandle.IsValid())
	{
		DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.AGVDeativateReverb"), STAT_AGVDeactivateReverb, STATGROUP_AudioThreadCommands);

		FAudioThread::RunCommandOnGameThread([AudioDeviceHandle]() mutable
		{
			if (AudioDeviceHandle.IsValid())
			{
				static FName NAME_AGVReverb(TEXT("AGVReverb"));
				AudioDeviceHandle->DeactivateReverbEffect(NAME_AGVReverb);
			}
		}, GET_STATID(STAT_AGVDeactivateReverb));
	}
}

UReverbVolumeComponent::UReverbVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PayloadType = AudioGameplay::EComponentPayload::AGCP_Listener;
	bAutoActivate = true;
}

void UReverbVolumeComponent::SetReverbSettings(const FReverbSettings& NewReverbSettings)
{
	ReverbSettings = NewReverbSettings;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> UReverbVolumeComponent::FactoryMutator() const
{
	return MakeShared<FProxyMutator_Reverb>();
}

void UReverbVolumeComponent::CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	TSharedPtr<FProxyMutator_Reverb> ReverbMutator = StaticCastSharedPtr<FProxyMutator_Reverb>(Mutator);
	ReverbMutator->ReverbSettings = ReverbSettings;
}

