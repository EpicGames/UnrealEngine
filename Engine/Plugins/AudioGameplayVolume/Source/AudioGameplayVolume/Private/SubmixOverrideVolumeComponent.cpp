// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixOverrideVolumeComponent.h"
#include "AudioDevice.h"
#include "AudioGameplayVolumeLogs.h"
#include "AudioGameplayVolumeListener.h"
#include "Sound/SoundEffectSubmix.h"
#include "Sound/SoundSubmix.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixOverrideVolumeComponent)

constexpr TCHAR FProxyMutator_SubmixOverride::MutatorSubmixOverrideName[];

FProxyMutator_SubmixOverride::FProxyMutator_SubmixOverride()
{
	MutatorName = MutatorSubmixOverrideName;
}

void FProxyMutator_SubmixOverride::Apply(FAudioGameplayVolumeListener& Listener) const
{
	check(IsInAudioThread());

	FAudioDeviceHandle AudioDeviceHandle = FAudioDeviceManager::Get()->GetAudioDevice(Listener.GetOwningDeviceId());
	if (AudioDeviceHandle.IsValid())
	{
		FSoundEffectSubmixInitData InitData;
		InitData.DeviceID = AudioDeviceHandle.GetDeviceID();
		InitData.SampleRate = AudioDeviceHandle->GetSampleRate();
		TArray<FSoundEffectSubmixPtr> SubmixEffectPresetChainOverride;

		for (const FAudioVolumeSubmixOverrideSettings& OverrideSettings : SubmixOverrideSettings)
		{
			if (OverrideSettings.Submix && OverrideSettings.SubmixEffectChain.Num() > 0)
			{
				UE_LOGF(AudioGameplayVolumeLog, Verbose, "SubmixOverride Apply - Submix: %ls, Priority: %d, VolumeID: %08x, EffectChain: %d effects, CrossfadeTime: %.3f",
					*OverrideSettings.Submix->GetName(), Priority, VolumeID, OverrideSettings.SubmixEffectChain.Num(), OverrideSettings.CrossfadeTime);

				// Build the instances of the new submix preset chain override
				for (USoundEffectSubmixPreset* SubmixEffectPreset : OverrideSettings.SubmixEffectChain)
				{
					if (SubmixEffectPreset)
					{
						UE_LOGF(AudioGameplayVolumeLog, Verbose, "  Effect: %ls", *SubmixEffectPreset->GetName());

						InitData.PresetSettings = nullptr;
						InitData.ParentPresetUniqueId = SubmixEffectPreset->GetUniqueID();

						TSoundEffectSubmixPtr SoundEffectSubmix = USoundEffectPreset::CreateInstance<FSoundEffectSubmixInitData, FSoundEffectSubmix>(InitData, *SubmixEffectPreset);
						SoundEffectSubmix->SetEnabled(true);
						SubmixEffectPresetChainOverride.Add(SoundEffectSubmix);
					}
				}

				AudioDeviceHandle->SetSubmixEffectChainOverride(OverrideSettings.Submix, SubmixEffectPresetChainOverride, OverrideSettings.CrossfadeTime);
				SubmixEffectPresetChainOverride.Reset();
			}
		}
	}
}

void FProxyMutator_SubmixOverride::Remove(FAudioGameplayVolumeListener& Listener) const
{
	check(IsInAudioThread());

	FAudioDeviceHandle AudioDeviceHandle = FAudioDeviceManager::Get()->GetAudioDevice(Listener.GetOwningDeviceId());
	if (AudioDeviceHandle.IsValid())
	{
		// Clear out any previous submix effect chain overrides
		for (const FAudioVolumeSubmixOverrideSettings& OverrideSettings : SubmixOverrideSettings)
		{
			UE_LOGF(AudioGameplayVolumeLog, Verbose, "SubmixOverride Remove - Submix: %ls, Priority: %d, VolumeID: %08x, CrossfadeTime: %.3f",
				OverrideSettings.Submix ? *OverrideSettings.Submix->GetName() : TEXT("null"), Priority, VolumeID, OverrideSettings.CrossfadeTime);

			AudioDeviceHandle->ClearSubmixEffectChainOverride(OverrideSettings.Submix, OverrideSettings.CrossfadeTime);
		}
	}
}

USubmixOverrideVolumeComponent::USubmixOverrideVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PayloadType = AudioGameplay::EComponentPayload::AGCP_Listener;
	bAutoActivate = true;
}

void USubmixOverrideVolumeComponent::SetSubmixOverrideSettings(const TArray<FAudioVolumeSubmixOverrideSettings>& NewSubmixOverrideSettings)
{
	SubmixOverrideSettings = NewSubmixOverrideSettings;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> USubmixOverrideVolumeComponent::FactoryMutator() const
{
	return MakeShared<FProxyMutator_SubmixOverride>();
}

void USubmixOverrideVolumeComponent::CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	TSharedPtr<FProxyMutator_SubmixOverride> SubmixMutator = StaticCastSharedPtr<FProxyMutator_SubmixOverride>(Mutator);
	SubmixMutator->SubmixOverrideSettings = SubmixOverrideSettings;
}

