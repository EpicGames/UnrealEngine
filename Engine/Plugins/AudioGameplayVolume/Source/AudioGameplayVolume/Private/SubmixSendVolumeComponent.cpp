// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmixSendVolumeComponent.h"
#include "Sound/AudioVolume.h"
#include "Sound/SoundSubmixSend.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SubmixSendVolumeComponent)

constexpr TCHAR FProxyMutator_SubmixSend::MutatorSubmixSendName[];

FProxyMutator_SubmixSend::FProxyMutator_SubmixSend()
{
	MutatorName = MutatorSubmixSendName;
}

void FProxyMutator_SubmixSend::Apply(FAudioProxyActiveSoundParams& Params) const
{
	check(IsInAudioThread());

	// Determine location state.  Inside if any of the following conditions are met:
	// spatialization disabled on the active sound, we're in the same volume as the listener,
	// or if the active sound's interior settings are still 'default' (bUsingWorldSettings)
	EAudioVolumeLocationState LocationState = EAudioVolumeLocationState::OutsideTheVolume;
	if (Params.bListenerInVolume || !Params.bAllowSpatialization || Params.bUsingWorldSettings)
	{
		LocationState = EAudioVolumeLocationState::InsideTheVolume;
	}

	if (SubmixSendSettings.Num() > 0)
	{
		for (const FAudioVolumeSubmixSendSettings& SendSetting : SubmixSendSettings)
		{
			if (SendSetting.ListenerLocationState == LocationState)
			{
				for (const FSoundSubmixSendInfo& SendInfo : SendSetting.SubmixSends)
				{
					Params.SoundSubmixSends.Add(SendInfo);
				}
			}
		}
	}
}

USubmixSendVolumeComponent::USubmixSendVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PayloadType = AudioGameplay::EComponentPayload::AGCP_ActiveSound;
	bAutoActivate = true;
}

void USubmixSendVolumeComponent::SetSubmixSendSettings(const TArray<FAudioVolumeSubmixSendSettings>& NewSubmixSendSettings)
{
	SubmixSendSettings = NewSubmixSendSettings;

	// Let the parent volume know we've changed
	NotifyDataChanged();
}

TSharedPtr<FProxyVolumeMutator> USubmixSendVolumeComponent::FactoryMutator() const
{
	return MakeShared<FProxyMutator_SubmixSend>();
}

void USubmixSendVolumeComponent::CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const
{
	TSharedPtr<FProxyMutator_SubmixSend> SubmixMutator = StaticCastSharedPtr<FProxyMutator_SubmixSend>(Mutator);
	SubmixMutator->SubmixSendSettings = SubmixSendSettings;
}

