// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayVolumeMutator.h"
#include "AttenuationVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

/**
 *  FProxyMutator_Attenuation - An audio thread representation of occlusion settings (volume attenuation)
 */
class FProxyMutator_Attenuation : public FProxyVolumeMutator
{
public:

	FProxyMutator_Attenuation();
	virtual ~FProxyMutator_Attenuation() = default;

	float ExteriorVolume = 1.0f;
	float ExteriorTime = 0.5f;
	float InteriorVolume = 1.0f;
	float InteriorTime = 0.5f;

	virtual void Apply(FInteriorSettings& InteriorSettings) const override;
	virtual void Apply(FAudioProxyActiveSoundParams& Params) const override;

protected:

	constexpr static const TCHAR MutatorAttenuationName[] = TEXT("Attenuation");
};

/**
 *  Component driving the volume of sounds playing inside or outside the associated AGV according to where
 *  the audio listener lays relative to this spatial volume.
 *  Note: This component is only compatible with Audio Toggles using a "Listener in Primitives" or "Listener in Range" condition.
 */
UCLASS(MinimalAPI, Blueprintable, Config = Game, ClassGroup = ("AudioGameplayVolume"), meta = (BlueprintSpawnableComponent, DisplayName = "Interior-Exterior Attenuation"))
class UAttenuationVolumeComponent : public UAudioGameplayVolumeMutator
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UAttenuationVolumeComponent() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetExteriorVolume(float Volume, float InterpolateTime);

	float GetExteriorVolume() const { return ExteriorVolume; }
	float GetExteriorTime() const { return ExteriorTime; }

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetInteriorVolume(float Volume, float InterpolateTime);

	float GetInteriorVolume() const { return InteriorVolume; }
	float GetInteriorTime() const { return InteriorTime; }

private:

	//~ Begin UAudioGameplayVolumeMutator interface
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const override;
	UE_API virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const override;
	//~ End UAudioGameplayVolumeMutator interface

	// The desired volume of sounds outside the volume when the player is inside the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeAttenuation", meta = (AllowPrivateAccess = "true"))
	float ExteriorVolume = 1.0f;

	// The time over which to interpolate from the current volume to the desired volume of sounds outside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeAttenuation", meta = (AllowPrivateAccess = "true"))
	float ExteriorTime = 0.5f;

	// The desired volume of sounds inside the volume when the player is outside the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeAttenuation", meta = (AllowPrivateAccess = "true"))
	float InteriorVolume = 1.0f;

	// The time over which to interpolate from the current volume to the desired volume of sounds inside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeAttenuation", meta = (AllowPrivateAccess = "true"))
	float InteriorTime = 0.5f;
};

#undef UE_API
