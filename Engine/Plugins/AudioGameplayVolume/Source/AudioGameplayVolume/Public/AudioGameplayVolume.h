// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Volume.h"
#include "AudioGameplayVolume.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

enum class ETeleportType : uint8;
enum class EUpdateTransformFlags : int32;


DECLARE_DYNAMIC_MULTICAST_DELEGATE(FAudioGameplayVolumeEvent);

// Forward Declarations 
class UAudioGameplayVolumeComponent;
class UAudioGameplayVolumeSubsystem;

/**
 * AudioGameplayVolume - A spatial volume used to notify audio gameplay systems when the nearest audio listener
   enters or exits the volume. Additionally, these volumes can influence audio sources depending on the relative
   position of the listener.

   NOTE: Will only impact audio sources that have "apply ambient volumes" set on their sound class.
 */
UCLASS(MinimalAPI, hidecategories = (Advanced, Attachment, Collision, Volume))
class AAudioGameplayVolume : public AVolume
{
	GENERATED_UCLASS_BODY()

private:

	// A representation of this volume for the audio thread
	UPROPERTY(Transient)
	TObjectPtr<UAudioGameplayVolumeComponent> AGVComponent = nullptr;

	// Whether this volume is currently enabled.  Disabled volumes will not have a volume proxy, 
	// and therefore will not be considered for intersection checks.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, ReplicatedUsing = OnRep_bEnabled, Category = "AudioGameplay", Meta = (AllowPrivateAccess = "true"))
	bool bEnabled = true;

public:

	bool GetEnabled() const { return bEnabled; }
	
	/** Sets whether the volume is enabled */
	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetEnabled(bool bEnable);

	/** Blueprint event for listener enter */
	UFUNCTION(BlueprintNativeEvent, Category = Events)
	UE_API void OnListenerEnter();

	/** Blueprint event for listener exit */
	UFUNCTION(BlueprintNativeEvent, Category = Events)
	UE_API void OnListenerExit();

	UPROPERTY(BlueprintAssignable, Category = Events)
	FAudioGameplayVolumeEvent OnListenerEnterEvent;

	UPROPERTY(BlueprintAssignable, Category = Events)
	FAudioGameplayVolumeEvent OnListenerExitEvent;

	//~ Begin UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual bool ShouldCheckCollisionComponentForErrors() const override { return false; }
#endif // WITH_EDITOR
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UObject Interface

	//~ Begin AActor Interface
	UE_API virtual void PostInitializeComponents() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
	//~ End AActor Interface

	/** Called by a child component to notify our proxy may need updating */
	UE_API void OnComponentDataChanged();

	UE_API bool CanSupportProxy() const;

protected:

	UFUNCTION()
	UE_API virtual void OnRep_bEnabled();

	UE_API void TransformUpdated(USceneComponent* InRootComponent, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	UE_API void AddProxy() const;
	UE_API void RemoveProxy() const;
	UE_API void UpdateProxy() const;
};

#undef UE_API
