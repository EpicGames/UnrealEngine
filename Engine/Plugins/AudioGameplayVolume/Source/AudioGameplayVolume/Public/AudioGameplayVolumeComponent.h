// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayComponent.h"
#include "Interfaces/IAudioGameplayVolumeInteraction.h"
#include "AudioGameplayVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

// Forward Declarations 
class UAudioGameplayVolumeProxy;
class UAudioGameplayVolumeSubsystem;

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnAudioGameplayVolumeProxyStateChange);

/**
 *  Component used to drive interaction with AudioGameplayVolumeSubsystem ("Audio Toggle" subsystem).
 *
 *  This is the component that makes the owning actor behave like an Audio Toggle, and where you can configure how a toggle
 *  can be active or inactive (through its Toggle Condition). Blueprints can bind arbitrary effects to the toggle becoming
 *  On or Off through this component's "On Toggled On" and "On Toggled Off" events. But you will usually want to add
 *  Audio
 *  
 *  NOTE: Do not inherit from this class, use UAudioGameplayVolumeComponentBase or UAudioGameplayVolumeMutator to create
 *  extendable functionality.
 */
UCLASS(MinimalAPI, Config = Game, ClassGroup = ("AudioGameplay"), meta = (BlueprintSpawnableComponent, IsBlueprintBase = false, DisplayName = "Audio Toggle"))
class UAudioGameplayVolumeComponent final : public UAudioGameplayComponent
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UAudioGameplayVolumeComponent() = default;

	UE_API void SetProxy(UAudioGameplayVolumeProxy* NewProxy);
	UAudioGameplayVolumeProxy* GetProxy() const { return Proxy; }

	/** Called by a component on same actor to notify our proxy may need updating */
	UE_API void OnComponentDataChanged();

	/** Called when the proxy is 'entered' - This is when the proxy goes from zero listeners to at least one. */
	UE_API void EnterProxy() const;

	/** Called when the proxy is 'exited' - This is when the proxy goes from at least one listeners to zero. */
	UE_API void ExitProxy() const;

	/** Blueprint event for proxy enter */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Toggled On"))
	FOnAudioGameplayVolumeProxyStateChange OnProxyEnter;

	/** Blueprint event for proxy exit */
	UPROPERTY(BlueprintAssignable, Category = Events, meta = (DisplayName = "On Toggled Off"))
	FOnAudioGameplayVolumeProxyStateChange OnProxyExit;

protected:

	/**
	 * The kind of condition that this Audio Toggle uses to evaluate its On/Off state.
	 * See the tooltip for each choice to see how they work and can be further configured.
	 */
	UPROPERTY(Instanced, EditAnywhere, BlueprintReadWrite, Category = "AudioGameplay", Meta = (ShowOnlyInnerProperties, AllowPrivateAccess = "true", DisplayName = "Toggle Condition"))
	TObjectPtr<UAudioGameplayVolumeProxy> Proxy = nullptr;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	//~ Begin UActorComponent Interface
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	//~ End UActorComponent Interface

	//~ Begin UAudioGameplayComponent Interface
	UE_API virtual void Enable() override;
	UE_API virtual void Disable() override;
	//~ End UAudioGameplayComponent Interface

	UE_API void AddProxy() const;
	UE_API void RemoveProxy() const;
	UE_API void UpdateProxy() const;

	UE_API UAudioGameplayVolumeSubsystem* GetSubsystem() const;
};

/**
 *  UAudioGameplayVolumeComponentBase - Blueprintable component used to craft custom functionality with AudioGameplayVolumes.
 *   NOTE: Inherit from this class to get easy access to OnListenerEnter and OnListenerExit Blueprint Events
 */
UCLASS(MinimalAPI, Blueprintable, ClassGroup = ("AudioGameplay"), hidecategories = (Tags, Collision), meta = (BlueprintSpawnableComponent))
class UAudioGameplayVolumeComponentBase : public UAudioGameplayComponent
																, public IAudioGameplayVolumeInteraction
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UAudioGameplayVolumeComponentBase() = default;
};

#undef UE_API
