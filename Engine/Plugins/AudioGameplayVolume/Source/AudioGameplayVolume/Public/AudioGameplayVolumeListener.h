// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/AudioVolume.h"

// Forward Declarations 
class FProxyVolumeMutator;
struct FAudioProxyMutatorSearchResult;

/**
 *  FInterpolatedInteriorSettings
 */
class FInterpolatedInteriorSettings
{
public:

	FInterpolatedInteriorSettings() = default;

	// Set a new interior settings target
	void Apply(const FInteriorSettings& NewSettings);

	// Interpolates between the source and end
	float Interpolate(double CurrentTime, double EndTime) const;

	// Updates the current state of the interior settings
	void UpdateInteriorValues();

	inline void SetInteriorStartTime(double InTime) { InteriorStartTime = InTime; }

	inline double GetInteriorStartTime() const { return InteriorStartTime; }

	inline float GetInteriorVolume() const { return InteriorSettings.InteriorVolume; }
	inline float GetInteriorVolumeInterp() const { return InteriorVolumeInterp; }

	inline float GetExteriorVolume() const { return InteriorSettings.ExteriorVolume; }
	inline float GetExteriorVolumeInterp() const { return ExteriorVolumeInterp; }

	inline float GetInteriorLPF() const { return InteriorSettings.InteriorLPF; }
	inline float GetInteriorLPFInterp() const { return InteriorLPFInterp; }

	inline float GetExteriorLPF() const { return InteriorSettings.ExteriorLPF; }
	inline float GetExteriorLPFInterp() const { return ExteriorLPFInterp; }

	inline bool IsWorldSettings() const { return InteriorSettings.bIsWorldSettings; }

protected:

	double InteriorStartTime = 0.0;

	double InteriorEndTime = 0.0;
	double ExteriorEndTime = 0.0;

	double InteriorLPFEndTime = 0.0;
	double ExteriorLPFEndTime = 0.0;

	float InteriorVolumeInterp = 0.f;
	float ExteriorVolumeInterp = 0.f;

	float InteriorLPFInterp = 0.f;
	float ExteriorLPFInterp = 0.f;

	FInteriorSettings InteriorSettings;
};

/**
 *  FAudioGameplayVolumeListener
 */
class FAudioGameplayVolumeListener
{
public:

	FAudioGameplayVolumeListener() = default;

	void Update(const FAudioProxyMutatorSearchResult& Result, const FVector& InPosition, uint32 InDeviceId);

	const TSet<uint32>& GetCurrentProxies() const { return CurrentProxies; }

	uint32 GetOwningDeviceId() const { return OwningDeviceId; }

	bool GetAffectedByLegacySystem() const { return bAffectedByLegacySystem; }
	void SetAffectedByLegacySystem(bool bIsAffected) { bAffectedByLegacySystem = bIsAffected; }

	const FInterpolatedInteriorSettings& GetInteriorSettings() const { return InteriorSettings; }

protected:

	FInterpolatedInteriorSettings InteriorSettings;

	uint32 OwningDeviceId = INDEX_NONE;
	FVector Position = FVector::ZeroVector;

	bool bAffectedByLegacySystem = false;

	TSet<uint32> CurrentProxies;
	TSet<uint32> PreviousProxies;

	TArray<TSharedPtr<FProxyVolumeMutator>> ActiveMutators;
};

