// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

AUDIOGAMEPLAYVOLUME_API DECLARE_LOG_CATEGORY_EXTERN(AudioGameplayVolumeLog, Log, All);
