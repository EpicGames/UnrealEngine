// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioDefines.h"
#include "AudioGameplayComponent.h"
#include "AudioGameplayVolumeMutator.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

struct FInteriorSettings;
struct FSoundSubmixSendInfo;

// Forward Declarations 
class FAudioGameplayVolumeListener;
struct FAudioGameplayActiveSoundInfo;

/**
 *  FAudioProxyActiveSoundParams - Helper struct for collecting info about the active sound from affecting proxy mutators
 */
struct FAudioProxyActiveSoundParams
{
	FAudioProxyActiveSoundParams() = delete;
	FAudioProxyActiveSoundParams(const FAudioGameplayActiveSoundInfo& SoundInfo, const FAudioGameplayVolumeListener& InListener);

	float SourceInteriorVolume = 1.f;

	float SourceInteriorLPF = MAX_FILTER_FREQUENCY;

	bool bAllowSpatialization = false;
	bool bUsingWorldSettings = false;
	bool bListenerInVolume = false;

	bool bAffectedByAttenuation = false;
	bool bAffectedByFilter = false;

	const FAudioGameplayVolumeListener& Listener;
	const FAudioGameplayActiveSoundInfo& Sound;

	TArray<FSoundSubmixSendInfo> SoundSubmixSends;

	void UpdateInteriorValues();
};

/**
 *  FProxyVolumeMutator - An audio thread representation of the payload for an AudioGameplayVolumeComponent.
 */
class FProxyVolumeMutator : public TSharedFromThis<FProxyVolumeMutator>
{
public:

	UE_API FProxyVolumeMutator();
	virtual ~FProxyVolumeMutator() = default;

	using PayloadFlags = AudioGameplay::EComponentPayload;

	virtual void Apply(FInteriorSettings& InteriorSettings) const;
	virtual void Apply(FAudioProxyActiveSoundParams& Params) const {}
	virtual void Apply(FAudioGameplayVolumeListener& Listener) const {}

	virtual void Remove(FAudioProxyActiveSoundParams& Params) const {}
	virtual void Remove(FAudioGameplayVolumeListener& Listener) const {}

	UE_API bool HasPayloadType(PayloadFlags InType) const;

	int32 Priority = INDEX_NONE;
	uint32 VolumeID = INDEX_NONE;
	uint32 WorldID = INDEX_NONE;

	FName MutatorName;

	PayloadFlags PayloadType;

protected:

	constexpr static const TCHAR MutatorBaseName[] = TEXT("MutatorBase");
};

/**
 *  UAudioGameplayVolumeMutator - These components are used for more complex interactions with AudioGameplayVolumes.
 *		Currently, components inheriting this base can affect interior settings as well as active sounds or the audio listener(s) inside the volume.
 *		See also: FilterVolumeComponent, AttenuationVolumeComponent, SubmixSendComponent, SubmixOverrideVolumeComponent, and ReverbVolumeComponent
 */
UCLASS(MinimalAPI, Abstract, HideDropdown, meta = (IsBlueprintBase = false))
class UAudioGameplayVolumeMutator : public UAudioGameplayComponent
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UAudioGameplayVolumeMutator() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetPriority(int32 InPriority);

	int32 GetPriority() const { return Priority; }

	/** Create and fill the appropriate proxy mutator for this component */
	UE_API virtual TSharedPtr<FProxyVolumeMutator> CreateMutator() const final;

	//~ Begin UObject Interface
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ End UObject Interface

protected:

	//~ Begin UAudioGameplayComponent interface 
	UE_API virtual void Enable() override;
	//~ End UAudioGameplayComponent interface
	
	/** Create this component's type of mutator */
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const;

	/** Override in child classes to copy additional data needed to mutators */
	virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const {}

	/** Notify our parent volume our proxy may need updating */
	UE_API void NotifyDataChanged() const;

	// The priority of this component.  In the case of overlapping volumes or multiple affecting components, the highest priority is chosen.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "AudioGameplay", Meta = (AllowPrivateAccess = "true"))
	int32 Priority = 0;

private:

	/** Called for you during mutator creation. See CopyAudioDataToMutator for adding data to derived classes */
	UE_API void CopyAudioDataToMutatorBase(TSharedPtr<FProxyVolumeMutator>& Mutator) const;
};

#undef UE_API
