// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayFlags.h"
#include "UObject/ObjectKey.h"

#include "AudioGameplayVolumeProxy.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

// Forward Declarations 
class FPrimitiveDrawInterface;
class FProxyVolumeMutator;
class FSceneView;
class UActorComponent;
class UAudioGameplayVolumeComponent;
class UPrimitiveComponent;
struct FBodyInstance;

/**
 *  Abstract condition type for Audio Toggles (UAudioGameplayVolumeComponent).
 *  
 *  The way Audio Toggles evaluate to On or Off is implemented in child classes by overriding
 *  the ContainsPosition function (because toggles were originally only implemented as the
 *  audio listener being inside or outside some 3D area (or "volume")).
 */
UCLASS(MinimalAPI, Abstract, EditInlineNew, HideDropdown)
class UAudioGameplayVolumeProxy : public UObject
{
	GENERATED_BODY()

public:

	using PayloadFlags = AudioGameplay::EComponentPayload;
	using ProxyMutatorList = TArray<TSharedPtr<FProxyVolumeMutator>>;

	virtual ~UAudioGameplayVolumeProxy() = default;

	UE_API virtual bool ContainsPosition(const FVector& Position) const;
	UE_API virtual void InitFromComponent(const UAudioGameplayVolumeComponent* Component);

	UE_API void CollectMutators(ProxyMutatorList& OutMutators) const;

	UE_API void AddPayloadType(PayloadFlags InType);
	UE_API bool HasPayloadType(PayloadFlags InType) const;

	UE_API uint32 GetVolumeID() const;
	UE_API uint32 GetWorldID() const;

	/** Used for debug visualization of UAudioGameplayVolumeProxy in the editor */
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) {}

protected:

	ProxyMutatorList ProxyVolumeMutators;

	uint32 VolumeID = INDEX_NONE;
	uint32 WorldID = INDEX_NONE;
	PayloadFlags PayloadType = PayloadFlags::AGCP_None;

#if !UE_BUILD_SHIPPING
public:
	FName GetOwnerName() const;
	
private:
	// Only for profiling use
	FObjectKey AGVOwnerKey;
#endif
};

/**
 *  Sets the Audio Toggle to On if the audio listener is in any of the primitives
 *  (as in, Primitive Components) of the owning actor, sets to Off otherwise.
 *  
 *  Note: The primitive components must have some physics enabled (for instance,
 *  OverlapAll - Primitives with the NoCollision profile are ignored).
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Audio Listener in Primitives"))
class UAGVPrimitiveComponentProxy : public UAudioGameplayVolumeProxy
{
	GENERATED_BODY()

public:

	virtual ~UAGVPrimitiveComponentProxy() = default;

	UE_API virtual bool ContainsPosition(const FVector& Position) const override;
	UE_API virtual void InitFromComponent(const UAudioGameplayVolumeComponent* Component) override;

protected:

	UPROPERTY(Transient)
	TArray<TObjectPtr<UPrimitiveComponent>> Primitives;
};

/**
 *  Sets the Audio Toggle state according to some arbitrary condition, regularly re-evaluated.
 *  
 *  If the owner actor implements IAudioGameplayCondition, sets the Audio Toggle to On
 *  when either ConditionMet or ConditionMet_Position is true, and sets it to Off when
 *  they are both false.
 *  If the owner actor does not implement it, looks for the first component on the
 *  actor found implementing IAudioGameplayCondition, then follows the same rules as above.
 */
UCLASS(MinimalAPI, meta = (DisplayName = "Arbitrary"))
class UAGVConditionProxy : public UAudioGameplayVolumeProxy
{
	GENERATED_BODY()

public:

	virtual ~UAGVConditionProxy() = default;

	UE_API virtual bool ContainsPosition(const FVector& Position) const override;
	UE_API virtual void InitFromComponent(const UAudioGameplayVolumeComponent* Component) override;

protected:

	UPROPERTY(Transient)
	TObjectPtr<const UObject> ObjectPtr;
};

#undef UE_API
