// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * DEPRECATED AudioGameplayVolumeProxyMutator.h.  Use the following files instead:
	 AudioGameplayVolumeSubsystem.h for FAudioProxyMutatorSearchResult
	 AudioGameplayVolumeSubsystem.h for FAudioProxyMutatorSearchObject
	 AudioGameplayVolumeMutator.h	for FAudioProxyMutatorPriorities
	 AudioGameplayVolumeMutator.h	for FAudioProxyActiveSoundParams
	 AudioGameplayVolumeMutator.h	for FProxyVolumeMutator
 */

#pragma once

// HEADER_UNIT_SKIP - Deprecated

UE_DEPRECATED_HEADER(5.1, "Use AudioGameplayVolumeSubsystem.h and AudioGameplayVolumeMutator.h instead of AudioGameplayVolumeProxyMutator.h.")

 // Include the new file so that the project still compiles, but has warnings
#include "AudioGameplayVolumeSubsystem.h"
#include "AudioGameplayVolumeMutator.h"
