// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActiveSoundUpdateInterface.h"
#include "AudioDefines.h"
#include "Subsystems/AudioEngineSubsystem.h"
#include "AudioGameplayFlags.h"
#include "AudioGameplayVolumeListener.h"
#include "Sound/ReverbSettings.h"
#include "AudioGameplayVolumeSubsystem.generated.h"

// Forward Declarations 
class UWorld;
class FProxyVolumeMutator;
class UAudioGameplayVolumeProxy;
class UAudioGameplayVolumeComponent;

/**
 *  FAudioGameplayActiveSoundInfo - Helper struct for caching active sound data
 */
struct FAudioGameplayActiveSoundInfo
{
	// Interior settings & interpolation values
	FInterpolatedInteriorSettings InteriorSettings;
	double LastUpdateTime = 0.0;

	// Current interior values
	float CurrentInteriorVolume = 1.f;
	float SourceInteriorVolume = 1.f;
	float CurrentInteriorLPF = MAX_FILTER_FREQUENCY;
	float SourceInteriorLPF = MAX_FILTER_FREQUENCY;

	TArray<TSharedPtr<FProxyVolumeMutator>> CurrentMutators;

	void Update(double ListenerInteriorStartTime);
};

/**
 *  FAudioGameplayProxyUpdateResult - Helper struct for updating game thread proxies
 */
struct FAudioGameplayProxyUpdateResult
{
	TSet<uint32> EnteredProxies;
	TSet<uint32> ExitedProxies;
};

/**
 *  FAudioProxyMutatorSearchResult - Results from a audio proxy mutator search (see below).
 */
struct FAudioProxyMutatorSearchResult
{
	TSet<uint32> VolumeSet;
	TArray<TSharedPtr<FProxyVolumeMutator>> MatchingMutators;
	FReverbSettings ReverbSettings;
	FInteriorSettings InteriorSettings;

	void Reset()
	{
		VolumeSet.Reset();
		MatchingMutators.Empty();
		ReverbSettings = FReverbSettings();
		InteriorSettings = FInteriorSettings();
	}
};

/**
 *  FAudioProxyMutatorSearchObject - Used for searching through proxy volumes to find relevant proxy mutators
 */
struct FAudioProxyMutatorSearchObject
{
	using PayloadFlags = AudioGameplay::EComponentPayload;

	// Search parameters
	uint32 WorldID = INDEX_NONE;
	FVector Location = FVector::ZeroVector;
	PayloadFlags PayloadType = PayloadFlags::AGCP_None;
	FAudioDeviceHandle AudioDeviceHandle;
	bool bAffectedByLegacySystem = false;
	bool bFilterPayload = true;
	bool bCollectMutators = true;
	bool bGetDefaultAudioSettings = true;

	void SearchVolumes(const TArray<UAudioGameplayVolumeProxy*>& ProxyVolumes, FAudioProxyMutatorSearchResult& OutResult);
};

/**
 *  FAudioGameplayVolumeProxyInfo - Holds information relating to which volumes our listeners are inside of
 */
class FAudioGameplayVolumeProxyInfo
{
public:

	FAudioGameplayVolumeProxyInfo() = default;

	/** Aggregate proxies from all listeners into a single world list */
	void Update(const TArray<FAudioGameplayVolumeListener>& VolumeListeners, FAudioGameplayProxyUpdateResult& OutResult);

	/** Add a listener's index to be checked on the next Update call */
	void AddListenerIndex(int32 ListenerIndex);

	/** Returns true if the volume contains at least one listener */
	bool IsVolumeInCurrentList(uint32 VolumeID) const;

protected:

	TSet<uint32> CurrentProxies;
	TSet<uint32> PreviousProxies;

	TArray<int32> ListenerIndexes;
};

/**
 *  UAudioGameplayVolumeSubsystem
 */
UCLASS()
class UAudioGameplayVolumeSubsystem : public UAudioEngineSubsystem
									, public IActiveSoundUpdateInterface
{
	GENERATED_BODY()

public: 

	virtual ~UAudioGameplayVolumeSubsystem() = default;

	//~ Begin USubsystem interface
	virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	virtual void Deinitialize() override;
	//~ End USubsystem interface

	//~ Begin UAudioEngineSubsystem interface
	virtual void Update() override;
	//~ End UAudioEngineSubsystem interface

	//~ Begin IActiveSoundUpdateInterface
	virtual void GatherInteriorData(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) override;
	virtual void ApplyInteriorSettings(const FActiveSound& ActiveSound, FSoundParseParameters& ParseParams) override;
	virtual void NotifyActiveSoundDeleting(const FActiveSound& ActiveSound) override;
	//~ End IActiveSoundUpdateInterface
	
	/** Add a volume to the system */
	void AddVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent);

	/** Update an existing volume in the system */
	void UpdateVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent);

	/** Remove a volume from the system */
	void RemoveVolumeComponent(const UAudioGameplayVolumeComponent* VolumeComponent);

protected:

	/** Returns true if we allow volumes from the world's type */
	bool DoesSupportWorld(UWorld* World) const;

	/** (Audio Thread Only) Add, Update, Remove ProxyVolumes */
	bool AddProxy(uint32 AudioGameplayVolumeID, uint32 WorldID);
	bool UpdateProxy(uint32 AudioGameplayVolumeID);
	bool RemoveProxy(uint32 AudioGameplayVolumeID);

	/** Returns true if a listener associated with WorldID is inside the volume (by ID) */
	bool IsAnyListenerInVolume(uint32 WorldID, uint32 VolumeID) const;

	/** Update the components driven by proxies on the game thread */
	void UpdateComponentsFromProxyInfo(const FAudioGameplayProxyUpdateResult& ProxyResults) const;

	/** Update our representation of audio listeners on the audio thread */
	void UpdateFromListeners();

	/** Generate a list of volume proxies from our component list */
	void GenerateVolumeProxyList();

	UPROPERTY(Transient)
	TArray<TObjectPtr<UAudioGameplayVolumeProxy>> TransientProxyList;

	// Components in our system. This map should only be modified by the game thread.
	UPROPERTY(Transient)
	TMap<uint32, TObjectPtr<const UAudioGameplayVolumeComponent>> AGVComponents;

	// Used to protect concurrent access to the AGVComponents map.
	FCriticalSection AGVComponentsCriticalSection;

	// List of Component IDs (for proxies) the audio thread is tracking
	TArray<uint32> KnownProxyIDs;

	// Audio thread representation of Listeners
	TArray<FAudioGameplayVolumeListener> AGVListeners;

	// A collection of data about currently playing active sounds, indexed by the sound's unique ID
	TMap<uint32, FAudioGameplayActiveSoundInfo> ActiveSoundData;

	// A collection of listener & volume intersection data, by worldID
	TMap<uint32, FAudioGameplayVolumeProxyInfo> WorldProxyLists;

	// Time since last update call, in seconds
	float TimeSinceUpdate = 0.f;

	// Time when next update happens relative to last update
	float NextUpdateDeltaTime = 0.f;

	// The number of proxy volumes from the previous update
	int32 PreviousProxyCount = 0;

	// Force an enter / exit of a volume to handle data changing while a proxy is active
	bool bHasStaleProxy = false;
};
