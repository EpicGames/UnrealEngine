// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayVolumeMutator.h"
#include "FilterVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

/**
 *  FProxyMutator_Filter - An audio thread representation of occlusion settings (volume filter)
 */
class FProxyMutator_Filter : public FProxyVolumeMutator
{
public:

	FProxyMutator_Filter();
	virtual ~FProxyMutator_Filter() = default;

	float ExteriorLPF = MAX_FILTER_FREQUENCY;
	float ExteriorLPFTime = 0.5f;
	float InteriorLPF = MAX_FILTER_FREQUENCY;
	float InteriorLPFTime = 0.5f;

	virtual void Apply(FInteriorSettings& InteriorSettings) const override;
	virtual void Apply(FAudioProxyActiveSoundParams& Params) const override;

protected:

	constexpr static const TCHAR MutatorFilterName[] = TEXT("Filter");
};

/**
 *  UFilterVolumeComponent - Audio Gameplay Volume component for occlusion settings (volume filter)
 */
UCLASS(MinimalAPI, Blueprintable, Config = Game, ClassGroup = ("AudioGameplayVolume"), meta = (BlueprintSpawnableComponent, DisplayName = "Filter"))
class UFilterVolumeComponent : public UAudioGameplayVolumeMutator
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UFilterVolumeComponent() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetExteriorLPF(UPARAM(DisplayName = "Frequency") float Volume, float InterpolateTime);

	float GetExteriorLPF() const { return ExteriorLPF; }
	float GetExteriorLPFTime() const { return ExteriorLPFTime; }

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetInteriorLPF(UPARAM(DisplayName = "Frequency") float Volume, float InterpolateTime);

	float GetInteriorLPF() const { return InteriorLPF; }
	float GetInteriorLPFTime() const { return InteriorLPFTime; }

private:

	//~ Begin UAudioGameplayVolumeMutator interface
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const override;
	UE_API virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const override;
	//~ End UAudioGameplayVolumeMutator interface

	// The desired LPF frequency cutoff (in hertz) of sounds outside the volume when the player is inside the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeFilter", meta = (AllowPrivateAccess = "true"))
	float ExteriorLPF = MAX_FILTER_FREQUENCY;

	// The time over which to interpolate from the current LPF to the desired LPF of sounds outside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeFilter", meta = (AllowPrivateAccess = "true"))
	float ExteriorLPFTime = 0.5f;

	// The desired LPF frequency cutoff (in hertz) of sounds inside the volume when the player is outside the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeFilter", meta = (AllowPrivateAccess = "true"))
	float InteriorLPF = MAX_FILTER_FREQUENCY;

	// The time over which to interpolate from the current LPF to the desired LPF of sounds inside the volume when the player enters the volume
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "VolumeFilter", meta = (AllowPrivateAccess = "true"))
	float InteriorLPFTime = 0.5f;
};

#undef UE_API
