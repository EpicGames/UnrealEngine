// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayVolumeMutator.h"
#include "Sound/ReverbSettings.h"
#include "ReverbVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

/**
 *  FProxyMutator_Reverb - An audio thread representation of the reverb volume component
 */
class FProxyMutator_Reverb : public FProxyVolumeMutator
{
public:

	FProxyMutator_Reverb();
	virtual ~FProxyMutator_Reverb() = default;

	FReverbSettings ReverbSettings;

	virtual void Apply(FAudioGameplayVolumeListener& Listener) const override;
	virtual void Remove(FAudioGameplayVolumeListener& Listener) const override;

protected:

	constexpr static const TCHAR MutatorReverbName[] = TEXT("Reverb");
};

/**
 *  UReverbVolumeComponent - Audio Gameplay Volume component for reverb settings
 */
UCLASS(MinimalAPI, Blueprintable, Config = Game, ClassGroup = ("AudioGameplayVolume"), meta = (BlueprintSpawnableComponent, DisplayName = "Reverb"))
class UReverbVolumeComponent : public UAudioGameplayVolumeMutator
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~UReverbVolumeComponent() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetReverbSettings(const FReverbSettings& NewReverbSettings);

	const FReverbSettings& GetReverbSettings() const { return ReverbSettings; }

	friend class FReverbVolumeComponentDetail;

private:

	//~ Begin UAudioGameplayVolumeMutator interface
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const override;
	UE_API virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const override;
	//~ End UAudioGameplayVolumeMutator interface

	/** Reverb settings to use with this component */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Reverb, meta = (AllowPrivateAccess = "true"))
	FReverbSettings ReverbSettings;
};

#undef UE_API
