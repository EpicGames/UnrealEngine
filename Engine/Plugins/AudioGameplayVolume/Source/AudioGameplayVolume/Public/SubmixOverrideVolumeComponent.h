// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayVolumeMutator.h"
#include "Sound/AudioVolume.h"
#include "SubmixOverrideVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

/**
 *  FProxyMutator_SubmixOverride - An audio thread representation of Submix Overrides
 */
class FProxyMutator_SubmixOverride : public FProxyVolumeMutator
{
public:

	FProxyMutator_SubmixOverride();
	virtual ~FProxyMutator_SubmixOverride() = default;

	TArray<FAudioVolumeSubmixOverrideSettings> SubmixOverrideSettings;

	virtual void Apply(FAudioGameplayVolumeListener& Listener) const override;
	virtual void Remove(FAudioGameplayVolumeListener& Listener) const override;

protected:

	constexpr static const TCHAR MutatorSubmixOverrideName[] = TEXT("SubmixOverride");
};

/**
 *  USubmixOverrideVolumeComponent - Audio Gameplay Volume component for submix effect chain overrides
 */
UCLASS(MinimalAPI, Blueprintable, Config = Game, ClassGroup = ("AudioGameplayVolume"), meta = (BlueprintSpawnableComponent, DisplayName = "Submix Override"))
class USubmixOverrideVolumeComponent : public UAudioGameplayVolumeMutator
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~USubmixOverrideVolumeComponent() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetSubmixOverrideSettings(const TArray<FAudioVolumeSubmixOverrideSettings>& NewSubmixOverrideSettings);

	const TArray<FAudioVolumeSubmixOverrideSettings>& GetSubmixOverrideSettings() const { return SubmixOverrideSettings; }

private:

	//~ Begin UAudioGameplayVolumeMutator interface
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const override;
	UE_API virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const override;
	//~ End UAudioGameplayVolumeMutator interface

	/** Submix effect chain override settings. Will override the effect chains on the given submixes */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submixes, meta = (AllowPrivateAccess = "true"))
	TArray<FAudioVolumeSubmixOverrideSettings> SubmixOverrideSettings;
};

#undef UE_API
