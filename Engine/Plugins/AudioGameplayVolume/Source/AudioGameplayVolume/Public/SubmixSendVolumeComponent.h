// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioGameplayVolumeMutator.h"
#include "Sound/AudioVolume.h"
#include "SubmixSendVolumeComponent.generated.h"

#define UE_API AUDIOGAMEPLAYVOLUME_API

/**
 *  FProxyMutator_SubmixSend - An audio thread representation of Submix Sends
 */
class FProxyMutator_SubmixSend : public FProxyVolumeMutator
{
public:

	FProxyMutator_SubmixSend();
	virtual ~FProxyMutator_SubmixSend() = default;

	TArray<FAudioVolumeSubmixSendSettings> SubmixSendSettings;

	virtual void Apply(FAudioProxyActiveSoundParams& Params) const override;

protected:

	constexpr static const TCHAR MutatorSubmixSendName[] = TEXT("SubmixSend");
};

/**
 *  USubmixSendVolumeComponent - Audio Gameplay Volume component for submix sends
 */
UCLASS(MinimalAPI, Blueprintable, Config = Game, ClassGroup = ("AudioGameplayVolume"), meta = (BlueprintSpawnableComponent, DisplayName = "Submix Send"))
class USubmixSendVolumeComponent : public UAudioGameplayVolumeMutator
{
	GENERATED_UCLASS_BODY()

public:

	virtual ~USubmixSendVolumeComponent() = default;

	UFUNCTION(BlueprintCallable, Category = "AudioGameplay")
	UE_API void SetSubmixSendSettings(const TArray<FAudioVolumeSubmixSendSettings>& NewSubmixSendSettings);

	const TArray<FAudioVolumeSubmixSendSettings>& GetSubmixSendSettings() const { return SubmixSendSettings; }

private:

	//~ Begin UAudioGameplayVolumeMutator interface
	UE_API virtual TSharedPtr<FProxyVolumeMutator> FactoryMutator() const override;
	UE_API virtual void CopyAudioDataToMutator(TSharedPtr<FProxyVolumeMutator>& Mutator) const override;
	//~ End UAudioGameplayVolumeMutator interface

	/** Submix send settings to use for this component. Allows audio to dynamically send to submixes based on source and listener locations (relative to parent volume.) */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Submixes, meta = (AllowPrivateAccess = "true"))
	TArray<FAudioVolumeSubmixSendSettings> SubmixSendSettings;
};

#undef UE_API
