// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AudioGameplayVolumeEditor : ModuleRules
{
	public AudioGameplayVolumeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"AudioGameplayVolume",
				"ToolMenus",
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"UnrealEd",
				"SlateCore",
				"PropertyEditor",
				"AudioGameplay",
			}
			);

		PrivateIncludePathModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
