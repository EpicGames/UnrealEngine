// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeComponentVisualizers.h"

#include "AudioGameplayVolumeComponent.h"
#include "AudioGameplayVolumeProxy.h"

void FAudioGameplayVolumeComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const UAudioGameplayVolumeComponent* AGVComponent = Cast<const UAudioGameplayVolumeComponent>(Component))
	{
		UAudioGameplayVolumeProxy* Proxy = AGVComponent->GetProxy();
		if (!AGVComponent->IsActive() || Proxy == nullptr)
		{
			return;
		}

		Proxy->DrawVisualization(Component, View, PDI);
	}
}
