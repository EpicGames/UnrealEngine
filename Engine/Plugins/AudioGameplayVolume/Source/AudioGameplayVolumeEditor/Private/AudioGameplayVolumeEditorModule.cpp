// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioGameplayVolumeEditorModule.h"
#include "AudioGameplayVolumeComponent.h"
#include "AudioGameplayVolumeComponentVisualizers.h"
#include "DetailCustomizations/ReverbVolumeComponentDetail.h"
#include "Editor/UnrealEdEngine.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "UnrealEdGlobals.h"

void FAudioGameplayVolumeEditorModule::StartupModule()
{
	constexpr bool bRegister = true;
	HandleCustomPropertyLayouts(bRegister);
	HandleComponentVisualizers(bRegister);
}

void FAudioGameplayVolumeEditorModule::ShutdownModule()
{
	constexpr bool bRegister = false;
	HandleCustomPropertyLayouts(bRegister);
	HandleComponentVisualizers(bRegister);
}

void FAudioGameplayVolumeEditorModule::HandleCustomPropertyLayouts(bool bRegister)
{
	// Register detail customizations
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

	if (bRegister)
	{
		PropertyModule.RegisterCustomClassLayout("ReverbVolumeComponent", FOnGetDetailCustomizationInstance::CreateStatic(&FReverbVolumeComponentDetail::MakeInstance));
	}
	else
	{
		PropertyModule.UnregisterCustomClassLayout("ReverbVolumeComponent");
	}
}

void FAudioGameplayVolumeEditorModule::HandleComponentVisualizers(bool bRegister)
{
	if (!GUnrealEd)
	{
		return;
	}

	if (bRegister)
	{
		RegisterComponentVisualizer(UAudioGameplayVolumeComponent::StaticClass()->GetFName(), MakeShared<FAudioGameplayVolumeComponentVisualizer>());
	}
	else
	{
		for (FName ClassName : RegisteredComponentVisualizers)
		{
			GUnrealEd->UnregisterComponentVisualizer(ClassName);
		}

		RegisteredComponentVisualizers.Reset(0);
	}
}

void FAudioGameplayVolumeEditorModule::RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
{
	check(GUnrealEd);
	GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
	RegisteredComponentVisualizers.Add(ComponentClassName);

	if (Visualizer.IsValid())
	{
		Visualizer->OnRegister();
	}
}

IMPLEMENT_MODULE(FAudioGameplayVolumeEditorModule, AudioGameplayVolumeEditor);
