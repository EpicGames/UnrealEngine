// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReverbVolumeComponentDetail.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"
#include "ReverbVolumeComponent.h"

void FReverbVolumeComponentDetail::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);

	const int32 CustomizedObjects = Objects.Num();
	ComponentToModify = nullptr;

	for (TWeakObjectPtr<UObject>& Object : Objects)
	{
		if (Object->IsA<UReverbVolumeComponent>())
		{
			ComponentToModify = Cast<UReverbVolumeComponent>(Object.Get());
			break;
		}
	}

	if (!ComponentToModify)
	{
		return;
	}

	TSharedPtr<IPropertyHandle> ReverbSettingsProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UReverbVolumeComponent, ReverbSettings));
	check(ReverbSettingsProperty->IsValidHandle());

	IDetailCategoryBuilder& ReverbCategory = DetailBuilder.EditCategory("Reverb");

	uint32 NumChildren = 0;
	ReverbSettingsProperty->GetNumChildren(NumChildren);
	TArray<TSharedRef<IPropertyHandle>> ChildPropertyHandles;

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ChildIndex++)
	{
		ChildPropertyHandles.Add(ReverbSettingsProperty->GetChildHandle(ChildIndex).ToSharedRef());
	}

	for (const TSharedRef<IPropertyHandle>& ChildPropertyHandle : ChildPropertyHandles)
	{
		// Leave alone if already customized
		if (ChildPropertyHandle->IsCustomized())
		{
			continue;
		}

		const FProperty* ChildProperty = ChildPropertyHandle->GetProperty();
		const FName ChildPropertyName = ChildProperty->GetFName();

		if (ChildPropertyName != GET_MEMBER_NAME_CHECKED(FReverbSettings, bApplyReverb) &&
			ChildPropertyName != GET_MEMBER_NAME_CHECKED(FReverbSettings, ReverbPluginEffect))
		{
			ReverbCategory.AddProperty(ChildPropertyHandle);
		}
	}

	DetailBuilder.HideProperty(ReverbSettingsProperty);
}
