// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/ObjectPtr.h"

struct FAssetData;
class UReverbVolumeComponent;
class IDetailCategoryBuilder;

class FReverbVolumeComponentDetail : public IDetailCustomization
{
public:

	static TSharedRef<IDetailCustomization> MakeInstance() { return MakeShared<FReverbVolumeComponentDetail>(); }

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	static void SortCategories(const TMap<FName, IDetailCategoryBuilder*>& AllCategories);
	bool ShouldFilterAsset(const FAssetData& AssetData);

	TObjectPtr<UReverbVolumeComponent> ComponentToModify = nullptr;
};
