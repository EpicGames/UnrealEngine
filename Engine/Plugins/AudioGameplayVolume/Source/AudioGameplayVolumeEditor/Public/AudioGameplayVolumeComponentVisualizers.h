// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

// Forward Declarations 
class UActorComponent;
class FSceneView;
class FPrimitiveDrawInterface;

class FAudioGameplayVolumeComponentVisualizer : public FComponentVisualizer
{
public:

	FAudioGameplayVolumeComponentVisualizer() = default;
	virtual ~FAudioGameplayVolumeComponentVisualizer() = default;

	//~ Begin FComponentVisualizer Interface
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	//~ End FComponentVisualizer Interface
};
