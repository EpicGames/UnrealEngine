// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointerFwd.h"

// Forward Declarations 
class FComponentVisualizer;

AUDIOGAMEPLAYVOLUMEEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(AudioGameplayVolumeEditor, Log, All);

class FAudioGameplayVolumeEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:

	void HandleCustomPropertyLayouts(bool bRegister);
	void HandleComponentVisualizers(bool bRegister);

	void RegisterComponentVisualizer(FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer);

	TArray<FName> RegisteredComponentVisualizers;
};
